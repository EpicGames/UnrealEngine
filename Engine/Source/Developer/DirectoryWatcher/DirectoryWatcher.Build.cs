// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DirectoryWatcher : ModuleRules
{
	public DirectoryWatcher(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.Add("Core");
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
