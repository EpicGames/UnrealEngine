// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryWatcherModule.h"
#include "DirectoryWatcherPrivate.h"
#include "DirectoryWatcherProxy.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE( FDirectoryWatcherModule, DirectoryWatcher );
DEFINE_LOG_CATEGORY(LogDirectoryWatcher);

void FDirectoryWatcherModule::StartupModule()
{
	DirectoryWatcher = new FDirectoryWatcherProxy();
}


void FDirectoryWatcherModule::ShutdownModule()
{
	if (DirectoryWatcher != NULL)
	{
		delete DirectoryWatcher;
		DirectoryWatcher = NULL;
	}
}

IDirectoryWatcher* FDirectoryWatcherModule::Get()
{
	return DirectoryWatcher;
}

void FDirectoryWatcherModule::RegisterExternalChanges(TArrayView<const FFileChangeData> FileChanges) const
{
	DirectoryWatcher->RegisterExternalChanges(FileChanges);
}
