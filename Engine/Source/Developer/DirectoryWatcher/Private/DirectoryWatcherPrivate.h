// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogDirectoryWatcher, Log, All);

#if PLATFORM_WINDOWS
#include "Windows/DirectoryWatchRequestWindows.h"
#include "Windows/DirectoryWatcherWindows.h"
#elif PLATFORM_MAC
#include "Mac/DirectoryWatchRequestMac.h"
#include "Mac/DirectoryWatcherMac.h"
#elif PLATFORM_LINUX
#include "Linux/DirectoryWatchRequestLinux.h"
#include "Linux/DirectoryWatcherLinux.h"
#else
#pragma message("FDirectoryWatcher is using DirectoryWatcherStub.h on this platform, which does nothing.")
#include "DirectoryWatcherStub.h"
#endif
