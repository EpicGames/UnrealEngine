// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryWatcherProxy.h"
#include "DirectoryWatcherPrivate.h"
#include "Async/TaskGraphInterfaces.h"

namespace DirectoryWatcherProxyUtil
{

FString GetAbsolutePath(const FString& InDirectory)
{
	FString AbsolutePath = FPaths::ConvertRelativePathToFull(InDirectory);
	AbsolutePath /= FString(); // Ensure a trailing slash
	return AbsolutePath;
}

}

FDirectoryWatcherProxy::FDirectoryWatcherProxy()
	: Inner(new FDirectoryWatcher())
	, bWatchMapPendingSort(false)
{
}

FDirectoryWatcherProxy::~FDirectoryWatcherProxy()
{
	delete Inner;
}

bool FDirectoryWatcherProxy::RegisterDirectoryChangedCallback_Handle(const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& Handle, uint32 Flags)
{
	if (Inner->RegisterDirectoryChangedCallback_Handle(Directory, InDelegate, Handle, Flags))
	{
		TArray<FWatchCallback>& WatchCallbacks = WatchMap.FindOrAdd(DirectoryWatcherProxyUtil::GetAbsolutePath(Directory));
		WatchCallbacks.Add(FWatchCallback{ InDelegate, Handle, Flags });
		bWatchMapPendingSort = true;
		return true;
	}

	return false;
}

bool FDirectoryWatcherProxy::UnregisterDirectoryChangedCallback_Handle(const FString& Directory, FDelegateHandle InHandle)
{
	const bool bSuccess = Inner->UnregisterDirectoryChangedCallback_Handle(Directory, InHandle);

	const FString WatchPath = DirectoryWatcherProxyUtil::GetAbsolutePath(Directory);
	if (TArray<FWatchCallback>* WatchCallbacks = WatchMap.Find(WatchPath))
	{
		WatchCallbacks->RemoveAll([&InHandle](const FWatchCallback& InWatchCallback)
		{
			return InWatchCallback.InnerHandle == InHandle;
		});
		if (WatchCallbacks->Num() == 0)
		{
			WatchMap.Remove(WatchPath);
		}
	}

	return bSuccess;
}

void FDirectoryWatcherProxy::Tick(float DeltaSeconds)
{
	Inner->Tick(DeltaSeconds);
	ProcessPendingChanges();
}

bool FDirectoryWatcherProxy::DumpStats()
{
	return Inner->DumpStats();
}

void FDirectoryWatcherProxy::RegisterExternalChanges(TArrayView<const FFileChangeData> FileChanges)
{
	if (IsInGameThread())
	{
		RegisterExternalChanges_GameThread(FileChanges);
	}
	else
	{
		FFunctionGraphTask::CreateAndDispatchWhenReady([this, FileChangesCopy = TArray<FFileChangeData>(FileChanges.GetData(), FileChanges.Num())]()
		{
			RegisterExternalChanges_GameThread(FileChangesCopy);
		}, TStatId(), nullptr, ENamedThreads::GameThread);
	}
}

void FDirectoryWatcherProxy::RegisterExternalChanges_GameThread(TArrayView<const FFileChangeData> FileChanges)
{
	PendingFileChanges.Append(FileChanges.GetData(), FileChanges.Num());
}

void FDirectoryWatcherProxy::ProcessPendingChanges()
{
	if (PendingFileChanges.Num() == 0)
	{
		return;
	}

	// Ensure the map is sorted correctly (by path length)
	if (bWatchMapPendingSort)
	{
		WatchMap.KeySort([](const FString& InPathOne, const FString& InPathTwo) -> bool
		{
			return InPathOne.Len() < InPathTwo.Len();
		});
		bWatchMapPendingSort = false;
	}

	TMap<const FWatchCallback*, TArray<FFileChangeData>> PendingNotifies;

	// Filter the changes to work out which of the the watchers we should notify
	for (const FFileChangeData& FileChange : PendingFileChanges)
	{
		// Note: FFileChangeData doesn't tell us whether the changed item is a file or directory (Mac and 
		// Linux know this information, but Windows does not), so this is a crude hack to try and guess
		const bool bIsDirectory = FPaths::GetExtension(FileChange.Filename).IsEmpty();

		FString FileChangePath = FPaths::ConvertRelativePathToFull(FileChange.Filename);
		if (!bIsDirectory)
		{
			FileChangePath = FPaths::GetPath(MoveTemp(FileChangePath));
		}
		FileChangePath /= FString(); // Ensure a trailing slash

		// Walk the map of watches looking for complete or partial matches
		for (const auto& WatchMapPair : WatchMap)
		{
			const FString& WatchPath = WatchMapPair.Key;

			// If this watch path is longer that the change path then we can skip it
			if (WatchPath.Len() > FileChangePath.Len())
			{
				// The map is sorted by path length, so we can bail once we find a watch path longer than our change path
				break;
			}

			// If the change path starts with this watch path then this is something we should potentially notify
			if (FileChangePath.StartsWith(WatchPath))
			{
				const bool bIsParentPath = WatchPath.Len() < FileChangePath.Len();
				for (const FWatchCallback& WatchCallback : WatchMapPair.Value)
				{
					// Should we notify this path based on its flags?
					if ((!bIsParentPath || (WatchCallback.WatchFlags & IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree) == 0) &&
						(!bIsDirectory || (WatchCallback.WatchFlags & IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges) != 0)
						)
					{
						TArray<FFileChangeData>& PendingNotifyFileChanges = PendingNotifies.FindOrAdd(&WatchCallback);
						PendingNotifyFileChanges.Add(FileChange);
					}
				}
			}
		}
	}
	PendingFileChanges.Reset();

	// Notify everything
	for (const auto& PendingNotifyPair : PendingNotifies)
	{
		PendingNotifyPair.Key->Delegate.ExecuteIfBound(PendingNotifyPair.Value);
	}
}
