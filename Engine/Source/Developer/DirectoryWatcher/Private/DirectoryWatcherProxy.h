// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectoryWatcher.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"

/**
 * Proxy around the real directory watcher.
 * Allows this proxy to process external file-system changes that aren't OS specific.
 */
class FDirectoryWatcherProxy : public IDirectoryWatcher
{
public:
	FDirectoryWatcherProxy();
	virtual ~FDirectoryWatcherProxy();

	virtual bool RegisterDirectoryChangedCallback_Handle(const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags) override;
	virtual bool UnregisterDirectoryChangedCallback_Handle(const FString& Directory, FDelegateHandle InHandle) override;
	virtual void Tick(float DeltaSeconds) override;
	virtual bool DumpStats() override;

	/** Register external changes that the OS file watcher couldn't detect (eg, a file changing in a UE sandbox) */
	void RegisterExternalChanges(TArrayView<const FFileChangeData> FileChanges);

private:
	/** Register external changes that the OS file watcher couldn't detect (eg, a file changing in a UE sandbox) */
	void RegisterExternalChanges_GameThread(TArrayView<const FFileChangeData> FileChanges);

	/** Process pending external changes that the OS file watcher couldn't detect (eg, a file changing in a UE sandbox) */
	void ProcessPendingChanges();

	/** Individual watch callback */
	struct FWatchCallback
	{
		/** Delegate to call when directory changes happen */
		FDirectoryChanged Delegate;

		/** Delegate handle after registering the internal directory watcher request */
		FDelegateHandle InnerHandle;

		/** Flags specified for this watch (see WatchOptions) */
		uint32 WatchFlags;
	};

	/** Internal directory watcher we act as a proxy for */
	IDirectoryWatcher* Inner;

	/** Array of pending file changes to notify on Tick */
	TArray<FFileChangeData> PendingFileChanges;

	/** Map from absolute directories to watch requests for those directories */
	TMap<FString, TArray<FWatchCallback>> WatchMap;

	/** True if the WatchMap is pending a sort */
	bool bWatchMapPendingSort;
};
