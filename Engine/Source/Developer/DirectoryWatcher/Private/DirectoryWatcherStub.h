// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDirectoryWatcher.h"


class FDirectoryWatcherStub : public IDirectoryWatcher
{
public:
	FDirectoryWatcherStub() = default;
	virtual ~FDirectoryWatcherStub() = default;

	virtual bool RegisterDirectoryChangedCallback_Handle (const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags) override { return false; }
	virtual bool UnregisterDirectoryChangedCallback_Handle(const FString& Directory, FDelegateHandle InHandle) override { return false; }
	virtual void Tick (float DeltaSeconds) override { }
	virtual bool DumpStats() override { return false; }
};

typedef FDirectoryWatcherStub FDirectoryWatcher;
