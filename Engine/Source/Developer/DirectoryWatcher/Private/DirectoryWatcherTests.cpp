// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogDirectoryWatcherTests, Log, All);

struct FDirectoryWatcherTestPayload
{
	FDelegateHandle WatcherDelegate;
	FString WorkingDir;
	TMap<FString, FFileChangeData::EFileChangeAction> ReportedChanges;

	FDirectoryWatcherTestPayload(const FString& InWorkingDir, uint32 Flags = 0)
		: WorkingDir(InWorkingDir)
	{
		IFileManager::Get().MakeDirectory(*WorkingDir, true);

		FDirectoryWatcherModule& Module = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		if (IDirectoryWatcher* DirectoryWatcher = Module.Get())
		{
			auto Callback = IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FDirectoryWatcherTestPayload::OnDirectoryChanged);
			DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(WorkingDir, Callback, WatcherDelegate, Flags);
		}
	}
	~FDirectoryWatcherTestPayload()
	{
		FDirectoryWatcherModule& Module = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
		if (IDirectoryWatcher* DirectoryWatcher = Module.Get())
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(WorkingDir, WatcherDelegate);
		}

		IFileManager::Get().DeleteDirectory(*WorkingDir, false, true);
	}

	void OnDirectoryChanged(const TArray<FFileChangeData>& InFileChanges)
	{
		for (const auto& Change : InFileChanges)
		{
			FString RelativeFilename = *FPaths::ConvertRelativePathToFull(Change.Filename) + FPaths::ConvertRelativePathToFull(WorkingDir).Len();

			UE_LOGF(LogDirectoryWatcherTests, Log, "File '%ls'. Code: %d.", *Change.Filename, (uint8)Change.Action);

			FFileChangeData::EFileChangeAction* Existing = ReportedChanges.Find(RelativeFilename);
			if (Existing)
			{
				switch (Change.Action)
				{
				case FFileChangeData::FCA_Added:
					*Existing = FFileChangeData::FCA_Modified;
					break;

				case FFileChangeData::FCA_Modified:
					// We ignore these since added + modified == added, and removed + modified = removed.
					break;

				case FFileChangeData::FCA_Removed:
					*Existing = FFileChangeData::FCA_Removed;
					break;
				}
			}
			else
			{
				ReportedChanges.Add(RelativeFilename, Change.Action);
			}
		}
	}
};

namespace DirectoryWatcherTests
{
	FString GetWorkingDirAsRelativePath()
	{
		return FPaths::AutomationTransientDir() / TEXT("DirectoryWatcher") / TEXT("");
	}

	FString GetWorkingDir()
	{
		return FPaths::ConvertRelativePathToFull(GetWorkingDirAsRelativePath()) / TEXT("");
	}
}

static const float TestTickDelay = 1.0f;
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherSimpleCreateTest, "System.Plugins.Directory Watcher.Simple Create", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherSimpleCreateTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* Filename = TEXT("created.tmp");

	// Start watching the directory
	TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir));

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test]{

		// Create a file and check that it got reported as created
		FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / Filename));

		AddCommand(new FDelayedFunctionLatentCommand([this, Test]{

			FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(Filename);

			if (!Action || *Action != FFileChangeData::FCA_Added)
			{
				UE_LOGF(LogDirectoryWatcherTests, Error, "New file '%ls' was not correctly reported as being added.", Filename);
			}

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherSimpleModifyTest, "System.Plugins.Directory Watcher.Simple Modify", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherSimpleModifyTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* Filename = TEXT("modified.tmp");

	// Create a file first
	FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / Filename));

	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir]{

		// Start watching the directory
		TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir));

		AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test]{

			// Manipulate the file
			FFileHelper::SaveStringToFile(TEXT("Some content"), *(WorkingDir / Filename));

			AddCommand(new FDelayedFunctionLatentCommand([this, Test]{

				FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(Filename);

				if (!Action || *Action != FFileChangeData::FCA_Modified)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "File '%ls' was not correctly reported as being modified.", Filename);
				}

			}, TestTickDelay));

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherSimpleDeleteTest, "System.Plugins.Directory Watcher.Simple Delete", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherSimpleDeleteTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* Filename = TEXT("removed.tmp");

	// Create a file first
	FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / Filename));

	// Start watching the directory
	TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir));

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test]{

		// Delete the file
		IFileManager::Get().Delete(*(WorkingDir / Filename));

		AddCommand(new FDelayedFunctionLatentCommand([this, Test]{

			FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(Filename);

			if (!Action || *Action != FFileChangeData::FCA_Removed)
			{
				UE_LOGF(LogDirectoryWatcherTests, Error, "File '%ls' was not correctly reported as being deleted.", Filename);
			}

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherSubFolderTest, "System.Plugins.Directory Watcher.Sub Folder", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherSubFolderTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* CreatedFilename = TEXT("sub_folder/created.tmp");
	static const TCHAR* ModifiedFilename = TEXT("sub_folder/modified.tmp");
	static const TCHAR* RemovedFilename = TEXT("sub_folder/removed.tmp");

	// Delete the created file, and create the modified and file to be deleted.
	IFileManager::Get().Delete(*( WorkingDir / CreatedFilename ));
	FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / ModifiedFilename));
	FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / RemovedFilename));

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir]{

		// Start watching the directory
		TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir));

		// Give the stream time to start up before doing the test
		AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test]{

			// Create a new file
			FFileHelper::SaveStringToFile(TEXT(""), *(WorkingDir / CreatedFilename));
			// Modify another file
			FFileHelper::SaveStringToFile(TEXT("Some content"), *(WorkingDir / ModifiedFilename));
			// Delete a file
			IFileManager::Get().Delete(*(WorkingDir / RemovedFilename));

			AddCommand(new FDelayedFunctionLatentCommand([this, Test]{

				FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(CreatedFilename);
				if (!Action || *Action != FFileChangeData::FCA_Added)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "File '%ls' was not correctly reported as being added.", CreatedFilename);
				}

				Action = Test->ReportedChanges.Find(ModifiedFilename);
				if (!Action || *Action != FFileChangeData::FCA_Modified)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "File '%ls' was not correctly reported as being modified.", ModifiedFilename);
				}

				Action = Test->ReportedChanges.Find(RemovedFilename);
				if (!Action || *Action != FFileChangeData::FCA_Removed)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "File '%ls' was not correctly reported as being deleted.", RemovedFilename);
				}

			}, TestTickDelay));

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherNewFolderTest, "System.Plugins.Directory Watcher.New Folder", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherNewFolderTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* CreatedDirectory = TEXT("created");
	static const TCHAR* RemovedDirectory = TEXT("removed");

	IFileManager::Get().DeleteDirectory(*( WorkingDir / CreatedDirectory ), true);

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir] {

		IFileManager::Get().MakeDirectory(*(WorkingDir / RemovedDirectory), true);

		// Start watching the directory
		TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir, IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges));

		// Give the stream time to start up before doing the test
		AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test] {

			IFileManager::Get().MakeDirectory(*(WorkingDir / CreatedDirectory), true);
			IFileManager::Get().DeleteDirectory(*(WorkingDir / RemovedDirectory), true);

			AddCommand(new FDelayedFunctionLatentCommand([this, Test] {

				FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(CreatedDirectory);
				if (!Action || *Action != FFileChangeData::FCA_Added)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "Folder '%ls' was not correctly reported as being added.", CreatedDirectory);
				}

				Action = Test->ReportedChanges.Find(RemovedDirectory);
				if (!Action || *Action != FFileChangeData::FCA_Removed)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "Folder '%ls' was not correctly reported as being removed.", RemovedDirectory);
				}

			}, TestTickDelay));

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherIgnoreSubtreeTest, "System.Plugins.Directory Watcher.Ignore Subtree", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherIgnoreSubtreeTest::RunTest(const FString& Parameters)
{
	const FString WorkingDir = DirectoryWatcherTests::GetWorkingDir();

	static const TCHAR* ChildDirectory = TEXT("child");
	static const TCHAR* GrandchildDirectory = TEXT("grandchild");

	IFileManager::Get().DeleteDirectory(*( WorkingDir / ChildDirectory ), true);

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir]{

		// Start watching the directory
		TSharedPtr<FDirectoryWatcherTestPayload> Test = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDir, IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree | IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges));

		// Give the stream time to start up before doing the test
		AddCommand(new FDelayedFunctionLatentCommand([this, WorkingDir, Test]{

			IFileManager::Get().MakeDirectory(*(WorkingDir / ChildDirectory), true);
			IFileManager::Get().MakeDirectory(*(WorkingDir / ChildDirectory / GrandchildDirectory), true);

			AddCommand(new FDelayedFunctionLatentCommand([this, Test]{

				FFileChangeData::EFileChangeAction* Action = Test->ReportedChanges.Find(ChildDirectory);
				if (!Action || *Action != FFileChangeData::FCA_Added)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "Folder '%ls' was not correctly reported as being added.", ChildDirectory);
				}

				Action = Test->ReportedChanges.Find(GrandchildDirectory);
				if (Action)
				{
					UE_LOGF(LogDirectoryWatcherTests, Error, "Changes to folder '%ls' (creation of subfolder '%ls') was reported in spite of us setting the mode 'ignore changes in subtree'.",
						ChildDirectory, GrandchildDirectory);
				}

			}, TestTickDelay));

		}, TestTickDelay));

	}, TestTickDelay));

	return true;
}

// Test dual directory watchers for the same directory using both relative and absolute paths.
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FDirectoryWatcherCreateDualWatchersTest, "System.Plugins.Directory Watcher.Create Dual Watchers", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FDirectoryWatcherCreateDualWatchersTest::RunTest(const FString& Parameters)
{
	static const TCHAR* Filename = TEXT("dual.tmp");

	const FString WorkingDirRelativePath = DirectoryWatcherTests::GetWorkingDirAsRelativePath();
	const FString WorkingDirFullPath = DirectoryWatcherTests::GetWorkingDir();
	const FString FileFullPath = WorkingDirFullPath / Filename;

	// Start watching the directory using relative path
	TSharedPtr<FDirectoryWatcherTestPayload> WatchRelativePath = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDirRelativePath));

	// Start watching the directory using full path
	TSharedPtr<FDirectoryWatcherTestPayload> WatchFullPath = MakeShareable(new FDirectoryWatcherTestPayload(WorkingDirFullPath));

	// Give the stream time to start up before doing the test
	AddCommand(new FDelayedFunctionLatentCommand([this, FileFullPath, WatchRelativePath, WatchFullPath] {

		// Create a file and check that it got reported as created by both watchers
		FFileHelper::SaveStringToFile(TEXT(""), *FileFullPath);

		AddCommand(new FDelayedFunctionLatentCommand([this, WatchRelativePath, WatchFullPath] {

			FFileChangeData::EFileChangeAction* Action = WatchRelativePath->ReportedChanges.Find(Filename);

			if (!Action || *Action != FFileChangeData::FCA_Added)
			{
				UE_LOGF(LogDirectoryWatcherTests, Error, "New file '%ls' was not correctly reported as being added by relative path watcher.", Filename);
			}

			Action = WatchFullPath->ReportedChanges.Find(Filename);

			if (!Action || *Action != FFileChangeData::FCA_Added)
			{
				UE_LOGF(LogDirectoryWatcherTests, Error, "New file '%ls' was not correctly reported as being added by full path watcher.", Filename);
			}

			}, TestTickDelay));

		}, TestTickDelay));

	return true;
}
