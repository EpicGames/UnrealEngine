// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileCache.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "HAL/Runnable.h"
#include "Misc/ScopeLock.h"
#include "Serialization/CustomVersion.h"
#include "DirectoryWatcherModule.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogFileCache, Log, All);

namespace DirectoryWatcher
{

template<typename T>
void ReadWithCustomVersions(FArchive& Ar, T& Data, ECustomVersionSerializationFormat CustomVersionFormat)
{
	int64 CustomVersionsOffset = 0;
	Ar << CustomVersionsOffset;

	const int64 DataStart = Ar.Tell();

	Ar.Seek(CustomVersionsOffset);

	// Serialize the custom versions
	FCustomVersionContainer Vers = Ar.GetCustomVersions();
	Vers.Serialize(Ar, CustomVersionFormat);
	Ar.SetCustomVersions(Vers);

	Ar.Seek(DataStart);

	Ar << Data;
}

template<typename T>
void WriteWithCustomVersions(FArchive& Ar, T& Data)
{
	const int64 CustomVersionsHeader = Ar.Tell();
	int64 CustomVersionsOffset = CustomVersionsHeader;
	// We'll come back later and fill this in
	Ar << CustomVersionsOffset;

	// Write out the data
	Ar << Data;

	CustomVersionsOffset = Ar.Tell();

	// Serialize the custom versions
	FCustomVersionContainer Vers = Ar.GetCustomVersions();
	Vers.Serialize(Ar);

	// Write out where the custom versions are in our header
	Ar.Seek(CustomVersionsHeader);
	Ar << CustomVersionsOffset;
}

/** Convert a FFileChangeData::EFileChangeAction into an EFileAction */
EFileAction ToFileAction(FFileChangeData::EFileChangeAction InAction)
{
	switch (InAction)
	{
	case FFileChangeData::FCA_Added: 	return EFileAction::Added;
	case FFileChangeData::FCA_Modified:	return EFileAction::Modified;
	case FFileChangeData::FCA_Removed:	return EFileAction::Removed;
	default:							return EFileAction::Modified;
	}
}

const FGuid FFileCacheCustomVersion::Key(0x8E7DDCB3, 0x80DA47BB, 0x9FD346A2, 0x93984DF6);
FCustomVersionRegistration GRegisterFileCacheVersion(FFileCacheCustomVersion::Key, FFileCacheCustomVersion::Latest, TEXT("FileCacheVersion"));

static const uint32 CacheFileMagicNumberOldCustomVersionFormat = 0x03DCCB00;
static const uint32 CacheFileMagicNumber = 0x03DCCB03;

static ECustomVersionSerializationFormat GetCustomVersionFormatForFileCache(uint32 MagicNumber)
{
	if (MagicNumber == CacheFileMagicNumberOldCustomVersionFormat)
	{
		return ECustomVersionSerializationFormat::Guids;
	}
	else
	{
		return ECustomVersionSerializationFormat::Optimized;
	}
}

/** Single runnable thread used to parse file cache directories without blocking the main thread */
struct FAsyncTaskThread : public FRunnable
{
	typedef TArray<TWeakPtr<IAsyncFileCacheTask, ESPMode::ThreadSafe>> FTaskArray;

	FAsyncTaskThread() : Thread(nullptr) {}

	/** Add a reader to this thread which will get ticked periodically until complete */
	void AddTask(TSharedPtr<IAsyncFileCacheTask, ESPMode::ThreadSafe> InTask)
	{
		FScopeLock Lock(&TaskArrayMutex);
		Tasks.Add(InTask);

		if (!Thread)
		{
			static int32 Index = 0;
			Thread = FRunnableThread::Create(this, *FString::Printf(TEXT("AsyncTaskThread_%d"), ++Index));
		}
	}

	/** Run this thread */
	virtual uint32 Run()
	{
		for(;;)
		{
			// Copy the array while we tick the readers
			FTaskArray Dupl;
			{
				FScopeLock Lock(&TaskArrayMutex);
				Dupl = Tasks;
			}

			// Tick each one for a second
			for (auto& Task : Dupl)
			{
				auto PinnedTask = Task.Pin();
				if (PinnedTask.IsValid())
				{
					PinnedTask->Tick(FTimeLimit(1));
				}
			}

			// Cleanup dead/finished Tasks
			FScopeLock Lock(&TaskArrayMutex);
			for (int32 Index = 0; Index < Tasks.Num(); )
			{
				auto Task = Tasks[Index].Pin();
				if (!Task.IsValid() || Task->IsComplete())
				{
					Tasks.RemoveAt(Index);
				}
				else
				{
					++Index;
				}
			}

			// Shutdown the thread if we've nothing left to do
			if (Tasks.Num() == 0)
			{
				Thread = nullptr;
				break;
			}
		}

		return 0;
	}

private:
	/** We start our own thread if one doesn't already exist. */
	FRunnableThread* Thread;

	/** Array of things that need ticking, and a mutex to protect them */
	FCriticalSection TaskArrayMutex;
	FTaskArray Tasks;
};

FAsyncTaskThread AsyncTaskThread;

/** Threading strategy for FAsyncFileHasher:
 *	The task is constructed on the main thread with its Data.
 *	The array 'Data' *never* changes size. The task thread moves along setting file hashes, while the main thread
 * 	trails behind accessing the completed entries. We should thus never have 2 threads accessing the same memory,
 *	except for the atomic 'CurrentIndex'
 */

FAsyncFileHasher::FAsyncFileHasher(TArray<FFilenameAndHash> InFilesThatNeedHashing)
	: Data(MoveTemp(InFilesThatNeedHashing)), NumReturned(0)
{
	// Read in files in 1MB chunks
	ScratchBuffer.SetNumUninitialized(1024 * 1024);
}

TArray<FFilenameAndHash> FAsyncFileHasher::GetCompletedData()
{
	// Don't need to lock here since the thread will never look at the array before CurrentIndex.
	TArray<FFilenameAndHash> Local;
	const int32 CompletedIndex = CurrentIndex.GetValue();

	if (NumReturned < CompletedIndex)
	{
		Local.Append(Data.GetData() + NumReturned, CompletedIndex - NumReturned);
		NumReturned = CompletedIndex;

		if (CompletedIndex == Data.Num())
		{
			Data.Empty();
			CurrentIndex.Set(0);
		}
	}

	return Local;
}

bool FAsyncFileHasher::IsComplete() const
{
	return CurrentIndex.GetValue() == Data.Num();
}

IAsyncFileCacheTask::EProgressResult FAsyncFileHasher::Tick(const FTimeLimit& Limit)
{
	for (; CurrentIndex.GetValue() < Data.Num(); )
	{
		const auto Index = CurrentIndex.GetValue();
		Data[Index].FileHash = FMD5Hash::HashFile(*Data[Index].AbsoluteFilename, &ScratchBuffer);

		CurrentIndex.Increment();

		if (Limit.Exceeded())
		{
			return EProgressResult::Pending;
		}
	}

	return EProgressResult::Finished;
}

/** Threading strategy for FAsyncDirectoryReader:
 *	The directory reader owns the cached and live state until it has completely finished. Once IsComplete() is true, the main thread can 
 *	have access to both the cached and farmed data.
 */


FAsyncDirectoryReader::FAsyncDirectoryReader(const FString& InDirectory, EPathType InPathType)
	: RootPath(InDirectory), PathType(InPathType)
{
	PendingDirectories.Add(InDirectory);
	LiveState.Emplace();

	StandardRootPath = RootPath;
	FPaths::MakeStandardFilename(StandardRootPath);
	StandardRootPath /= TEXT("");
}

TOptional<FDirectoryState> FAsyncDirectoryReader::GetLiveState()
{
	TOptional<FDirectoryState> OldState;

	if (ensureMsgf(IsComplete(), TEXT("Invalid property access from thread before task completion")))
	{
		Swap(OldState, LiveState);
	}

	return OldState;
}


TOptional<FDirectoryState> FAsyncDirectoryReader::GetCachedState()
{
	TOptional<FDirectoryState> OldState;

	if (ensureMsgf(IsComplete(), TEXT("Invalid property access from thread before task completion")))
	{
		Swap(OldState, CachedState);
	}

	return OldState;
}

bool FAsyncDirectoryReader::IsComplete() const
{
	return bIsComplete;
}

FAsyncDirectoryReader::EProgressResult FAsyncDirectoryReader::Tick(const FTimeLimit& TimeLimit)
{
	if (IsComplete())
	{
		return EProgressResult::Finished;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncDirectoryReader::Tick);
	auto& FileManager = IFileManager::Get();

	// Discover files
	for (int32 Index = 0; Index < PendingDirectories.Num(); ++Index)
	{
		ScanDirectory(PendingDirectories[Index]);
		if (TimeLimit.Exceeded())
		{
			// We've spent too long, bail
			PendingDirectories.RemoveAt(0, Index + 1, EAllowShrinking::No);
			return EProgressResult::Pending;
		}
		if (IsEngineExitRequested())
		{
			UE_LOGF(LogFileCache, Log, "Skipping file cache scan due to exit.");
			return EProgressResult::Finished;
		}
	}
	PendingDirectories.Empty();

	// Process files
	for (int32 Index = 0; Index < PendingFiles.Num(); ++Index)
	{
		const auto& File = PendingFiles[Index];

		// Store the file relative or absolute
		FString Filename = GetPathToStore(File);

		const auto Timestamp = FileManager.GetTimeStamp(*File);
	
		FMD5Hash MD5;
		if (CachedState.IsSet())
		{
			const FFileData* CachedData = CachedState->Files.Find(Filename);
			if (CachedData && CachedData->Timestamp == Timestamp && CachedData->FileHash.IsValid())
			{
				// Use the cached MD5 to avoid opening the file
				MD5 = CachedData->FileHash;
			}
		}

		if (!MD5.IsValid())
		{
			FilesThatNeedHashing.Emplace(File);
		}

		LiveState->Files.Emplace(MoveTemp(Filename), FFileData(Timestamp, MD5));

		if (TimeLimit.Exceeded())
		{
			// We've spent too long, bail
			PendingFiles.RemoveAt(0, Index + 1, EAllowShrinking::No);
			return EProgressResult::Pending;
		}
		if (IsEngineExitRequested())
		{
			UE_LOGF(LogFileCache, Log, "Skipping file cache scan due to exit.");
			return EProgressResult::Finished;
		}
	}
	PendingFiles.Empty();

	bIsComplete = true;

	UE_LOGF(LogFileCache, Log, "Scanning file cache for directory '%ls' took %.2fs", *RootPath, GetAge());
	return EProgressResult::Finished;
}

void FAsyncDirectoryReader::ScanDirectory(const FString& InDirectory)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncDirectoryReader::ScanDirectory);

	struct FVisitor : public IPlatformFile::FDirectoryVisitor
	{
		TArray<FString>* PendingFiles;
		TArray<FString>* PendingDirectories;
		FMatchRules* Rules;
		int32 RootPathLength;
		int32 StandardRootPathLength;

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			FString FileStr(FilenameOrDirectory);
			if (bIsDirectory)
			{
				PendingDirectories->Add(MoveTemp(FileStr));
			}
			else
			{
				const int32 Offset = FPaths::IsRelative(FileStr) ? StandardRootPathLength : RootPathLength;
				check(Offset <= FileStr.Len());
				if (Rules->IsFileApplicable(FilenameOrDirectory + Offset))
				{
					PendingFiles->Add(MoveTemp(FileStr));
				}
			}
			return true;
		}
	};

	FVisitor Visitor;
	Visitor.PendingFiles = &PendingFiles;
	Visitor.PendingDirectories = &PendingDirectories;
	Visitor.Rules = &LiveState->Rules;
	Visitor.RootPathLength = RootPath.Len();
	Visitor.StandardRootPathLength = StandardRootPath.Len();

	IFileManager::Get().IterateDirectory(*InDirectory, Visitor);
}

FString FAsyncDirectoryReader::GetPathToStore(const FString& InPath) const
{
	if (PathType == EPathType::Relative)
	{
		if (FPaths::IsRelative(InPath))
		{
			return InPath.RightChop(StandardRootPath.Len());
		}
		else
		{
			return InPath.RightChop(RootPath.Len());
		}
	}
	else
	{
		// TODO: Convert to absolute if not already absolute
		return InPath;
	}
}

FFileCache::FFileCache(const FFileCacheConfig& InConfig)
	: Config(InConfig)
	, bSavedCacheDirty(false)
	, LastFileHashGetTime(0)
{
	bPendingTransactionsDirty = true;

	// Ensure the directory has a trailing /
	Config.Directory /= TEXT("");

	// Store standardized copy of the directory
	ConfigDirectoryStandardized = Config.Directory;
	FPaths::MakeStandardFilename(ConfigDirectoryStandardized);
	ConfigDirectoryStandardized /= TEXT("");

	// bDetectMoves implies bRequireFileHashes
	Config.bRequireFileHashes = Config.bRequireFileHashes || Config.bDetectMoves;

	DirectoryReader = MakeShareable(new FAsyncDirectoryReader(Config.Directory, Config.PathType));
	DirectoryReader->SetMatchRules(Config.Rules);

	// Attempt to load an existing cache file
	auto ExistingCache = ReadCache();
	if (ExistingCache.IsSet())
	{
		DirectoryReader->UseCachedState(MoveTemp(ExistingCache.GetValue()));
	}

	AsyncTaskThread.AddTask(DirectoryReader);

	FDirectoryWatcherModule& Module = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	if (IDirectoryWatcher* DirectoryWatcher = Module.Get())
	{
		auto Callback = IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FFileCache::OnDirectoryChanged);
		if (!DirectoryWatcher->RegisterDirectoryChangedCallback_Handle(Config.Directory, Callback, WatcherDelegate))
		{
			UE_LOGF(LogFileCache, Display, "Failed registering directory watcher for folder '%ls'", *Config.Directory);
		}
	}
}

FFileCache::~FFileCache()
{
	UnbindWatcher();
	WriteCache();
}

void FFileCache::Destroy()
{
	// Delete the cache file, and clear out everything
	bSavedCacheDirty = false;
	if (!Config.CacheFile.IsEmpty())
	{
		IFileManager::Get().Delete(*Config.CacheFile, false, true, true);
	}

	DirectoryReader = nullptr;
	AsyncFileHasher = nullptr;
	DirtyFileHasher = nullptr;

	DirtyFiles.Empty();
	CachedDirectoryState = FDirectoryState();

	UnbindWatcher();
}

bool FFileCache::HasStartedUp() const
{
	return !DirectoryReader.IsValid() || DirectoryReader->IsComplete();
}

bool FFileCache::MoveDetectionInitialized() const
{
	if (!HasStartedUp())
	{
		return false;
	}
	else if (!Config.bDetectMoves)
	{
		return true;
	}
	else
	{
		// We don't check AsyncFileHasher->IsComplete() here because that doesn't necessarily mean we've harvested the results off the thread
		return !AsyncFileHasher.IsValid();
	}
}

const FFileData* FFileCache::FindFileData(FImmutableString InFilename) const
{
	if (!ensure(HasStartedUp()))
	{
		// It's invalid to call this while the cached state is still being updated on a thread.
		return nullptr;
	}

	return CachedDirectoryState.Files.Find(InFilename);
}

void FFileCache::UnbindWatcher()
{
	if (!WatcherDelegate.IsValid())
	{
		return;
	}

	if (FDirectoryWatcherModule* Module = FModuleManager::GetModulePtr<FDirectoryWatcherModule>(TEXT("DirectoryWatcher")))
	{
		if (IDirectoryWatcher* DirectoryWatcher = Module->Get())
		{
			DirectoryWatcher->UnregisterDirectoryChangedCallback_Handle(Config.Directory, WatcherDelegate);
		}
	}

	WatcherDelegate.Reset();
}

TOptional<FDirectoryState> FFileCache::ReadCache() const
{
	TOptional<FDirectoryState> Optional;
	if (!Config.CacheFile.IsEmpty())
	{		
		FArchive* Ar = IFileManager::Get().CreateFileReader(*Config.CacheFile);
		if (Ar)
		{
			// Serialize the magic number - the first iteration omitted version information, so we have a magic number to ignore this data
			uint32 MagicNumber = 0;
			*Ar << MagicNumber;

			if (MagicNumber == CacheFileMagicNumber || MagicNumber == CacheFileMagicNumberOldCustomVersionFormat)
			{
				FDirectoryState Result;
				ReadWithCustomVersions(*Ar, Result, GetCustomVersionFormatForFileCache(MagicNumber));

				Optional.Emplace(MoveTemp(Result));
			}

			Ar->Close();
			delete Ar;
		}
	}

	return Optional;
}

void FFileCache::WriteCache()
{
	if (bSavedCacheDirty && !Config.CacheFile.IsEmpty())
	{
		const FString ParentFolder = FPaths::GetPath(Config.CacheFile);
		if (!IFileManager::Get().DirectoryExists(*ParentFolder))
		{
			IFileManager::Get().MakeDirectory(*ParentFolder, true);	
		}

		// Write to a temp file to avoid corruption
		FString TempFile = Config.CacheFile + TEXT(".tmp");
		
		FArchive* Ar = IFileManager::Get().CreateFileWriter(*TempFile);
		if (ensureMsgf(Ar, TEXT("Unable to write file-cache for '%s' to '%s'."), *Config.Directory, *Config.CacheFile))
		{
			// Serialize the magic number
			uint32 MagicNumber = CacheFileMagicNumber;
			*Ar << MagicNumber;

			WriteWithCustomVersions(*Ar, CachedDirectoryState);

			Ar->Close();
			delete Ar;

			CachedDirectoryState.Files.Shrink();

			bSavedCacheDirty = false;

			const bool bMoved = IFileManager::Get().Move(*Config.CacheFile, *TempFile, true, true);
			if (!bMoved)
			{
				uint64 TotalDiskSpace = 0;
				uint64 FreeDiskSpace = 0;
				FPlatformMisc::GetDiskTotalAndFreeSpace(Config.CacheFile, TotalDiskSpace, FreeDiskSpace);
				ensureMsgf(bMoved, TEXT("Unable to move file-cache for '%s' from '%s' to '%s' (free disk space: %llu)"), *Config.Directory, *TempFile, *Config.CacheFile, FreeDiskSpace);
			}			
		}
	}
}

FString FFileCache::GetAbsolutePath(const FString& InTransactionPath) const
{
	if (Config.PathType == EPathType::Relative)
	{
		return Config.Directory / InTransactionPath;
	}
	else
	{
		return InTransactionPath;
	}
}

TOptional<FString> FFileCache::GetTransactionPath(const FString& InAbsolutePath) const
{
	FString Temp = FPaths::ConvertRelativePathToFull(InAbsolutePath);
	FString RelativePath(*Temp + Config.Directory.Len());

	// If it's a directory or is not applicable, ignore it
	if (!Temp.StartsWith(Config.Directory) || IFileManager::Get().DirectoryExists(*Temp) || !Config.Rules.IsFileApplicable(*RelativePath))
	{
		return TOptional<FString>();
	}

	if (Config.PathType == EPathType::Relative)
	{
		return MoveTemp(RelativePath);
	}
	else
	{
		return MoveTemp(Temp);
	}
}

void FFileCache::DiffDirtyFiles(TMap<FImmutableString, FFileData>& InDirtyFiles, TArray<FUpdateCacheTransaction>& OutTransactions, const FDirectoryState* InFileSystemState) const
{
	TMap<FImmutableString, FFileData> AddedFiles, ModifiedFiles;
	TSet<FImmutableString> RemovedFiles, InvalidDirtyFiles;

	auto& FileManager = IFileManager::Get();
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	for (const auto& Pair : InDirtyFiles)
	{
		const auto& File = Pair.Key;
		FString AbsoluteFilename = GetAbsolutePath(File.Get());
		
		const auto* CachedState = CachedDirectoryState.Files.Find(File);

		//Retrieve the timestamp directly instead of checking the existence of the file beforehand to save time
		//If the file doesn't exist the function will return FDateTime::MinValue to indicate this
		const FDateTime TimeStamp = InFileSystemState ? FDateTime::MinValue() : FileManager.GetTimeStamp(*AbsoluteFilename);
		const bool bFileExists = InFileSystemState ? InFileSystemState->Files.Find(File) != nullptr : TimeStamp != FDateTime::MinValue();
		if (bFileExists)
		{
			FFileData FileData;
			if (const auto* FoundData = InFileSystemState ? InFileSystemState->Files.Find(File) : nullptr)
			{
				FileData = *FoundData;
			}
			else
			{
				// The dirty file timestamp is the time that the file was dirtied, not necessarily its modification time
				FileData = FFileData(TimeStamp, Pair.Value.FileHash);
			}

			if (Config.bRequireFileHashes && !FileData.FileHash.IsValid())
			{
				// We don't have this file's hash yet. Temporarily ignore it.
				continue;
			}

			// Do we think it exists in the cache?
			if (CachedState)
			{
				// Custom logic overrides everything
				TOptional<bool> CustomResult = Config.CustomChangeLogic ? Config.CustomChangeLogic(File, FileData) : TOptional<bool>();
				if (CustomResult.IsSet())
				{
					if (CustomResult.GetValue())
					{
						ModifiedFiles.Add(File, FileData);
					}
					else
					{
						InvalidDirtyFiles.Add(File);
					}
				}
				// A file has changed if its hash is now different
				else if (Config.bRequireFileHashes &&
					Config.ChangeDetectionBits[FFileCacheConfig::FileHash] &&
					CachedState->FileHash != FileData.FileHash
					)
				{
					ModifiedFiles.Add(File, FileData);
				}
				// or the timestamp has changed
				else if (Config.ChangeDetectionBits[FFileCacheConfig::Timestamp] &&
					CachedState->Timestamp != FileData.Timestamp)
				{
					ModifiedFiles.Add(File, FileData);
				}
				else
				{
					// File hasn't changed
					InvalidDirtyFiles.Add(File);
				}
			}
			else
			{
				AddedFiles.Add(File, FileData);
			}
		}
		// We only report it as removed if it exists in the cache
		else if (CachedState)
		{
			RemovedFiles.Add(File);
		}
		else
		{
			// File doesn't exist, and isn't in the cache
			InvalidDirtyFiles.Add(File);
		}
	}

	// Remove any dirty files that aren't dirty
	for (auto& Filename : InvalidDirtyFiles)
	{
		InDirtyFiles.Remove(Filename);
	}

	// Rename / move detection
	if (Config.bDetectMoves)
	{
		bool bHavePendingHashes = false;

		// Remove any additions that don't have their hash generated yet
		for (auto AdIt = AddedFiles.CreateIterator(); AdIt; ++AdIt)
		{
			if (!AdIt.Value().FileHash.IsValid())
			{
				bHavePendingHashes = true;
				AdIt.RemoveCurrent();
			}
		}

		// We can only detect renames or moves for files that have had their file hash harvested.
		// If we can't find a valid move destination for this file, and we have pending hashes, ignore the removal until we can be sure it's not a move
		for (auto RemoveIt = RemovedFiles.CreateIterator(); RemoveIt; ++RemoveIt)
		{
			const auto* CachedState = CachedDirectoryState.Files.Find(*RemoveIt);
			if (CachedState && CachedState->FileHash.IsValid())
			{
				for (auto AdIt = AddedFiles.CreateIterator(); AdIt; ++AdIt)
				{
					if (AdIt.Value().FileHash == CachedState->FileHash)
					{
						// Found a move destination!
						OutTransactions.Add(FUpdateCacheTransaction(*RemoveIt, AdIt.Key(), AdIt.Value()));

						AdIt.RemoveCurrent();
						RemoveIt.RemoveCurrent();
						goto next;
					}
				}

				// We can't be sure this isn't a move (yet) so temporarily ignore this
				if (bHavePendingHashes)
				{
					RemoveIt.RemoveCurrent();
				}
			}

		next:
			continue;
		}
	}

	for (auto& RemovedFile : RemovedFiles)
	{
		OutTransactions.Add(FUpdateCacheTransaction(MoveTemp(RemovedFile), EFileAction::Removed));
	}
	// RemovedFiles is now bogus

	for (auto& Pair : AddedFiles)
	{
		OutTransactions.Add(FUpdateCacheTransaction(MoveTemp(Pair.Key), EFileAction::Added, Pair.Value));
	}
	// AddedFiles is now bogus
	
	for (auto& Pair : ModifiedFiles)
	{
		OutTransactions.Add(FUpdateCacheTransaction(MoveTemp(Pair.Key), EFileAction::Modified, Pair.Value));
	}
	// ModifiedFiles is now bogus
}

void FFileCache::UpdatePendingTransactions()
{
	if (bPendingTransactionsDirty)
	{
		PendingTransactions.Reset();

		DiffDirtyFiles(DirtyFiles, PendingTransactions);

		bPendingTransactionsDirty = false;
	}
}

void  FFileCache::IterateOutstandingChanges(TFunctionRef<bool(const FUpdateCacheTransaction&, const FDateTime&)> InIter) const
{
	for (const FUpdateCacheTransaction& Transaction : PendingTransactions)
	{
		FFileData FileData = DirtyFiles.FindRef(Transaction.Filename);
		if (!InIter(Transaction, FileData.Timestamp))
		{
			break;
		}
	}
}

TArray<FUpdateCacheTransaction> FFileCache::GetOutstandingChanges()
{
	// Harvest hashes first, since that may invalidate our pending transactions
	HarvestDirtyFileHashes();
	UpdatePendingTransactions();

	// Clear the set of dirty files since we're returning transactions for them now
	DirtyFiles.Empty();

	TArray<FUpdateCacheTransaction> ReturnVal = MoveTemp(PendingTransactions);
	PendingTransactions.Empty();
	return ReturnVal;
}

TArray<FUpdateCacheTransaction> FFileCache::FilterOutstandingChanges(TFunctionRef<bool(const FUpdateCacheTransaction&, const FDateTime&)> InPredicate)
{
	HarvestDirtyFileHashes();

	TArray<FUpdateCacheTransaction> AllTransactions;
	DiffDirtyFiles(DirtyFiles, AllTransactions, nullptr);

	// Filter the transactions based on the predicate
	TArray<FUpdateCacheTransaction> FilteredTransactions;
	for (auto& Transaction : AllTransactions)
	{
		FFileData FileData = DirtyFiles.FindRef(Transaction.Filename);

		// Timestamp is the time the file was dirtied, not necessarily the timestamp of the file
		if (InPredicate(Transaction, FileData.Timestamp))
		{
			DirtyFiles.Remove(Transaction.Filename);
			if (Transaction.Action == EFileAction::Moved)
			{
				DirtyFiles.Remove(Transaction.MovedFromFilename);
			}

			FilteredTransactions.Add(MoveTemp(Transaction));
		}
	}

	bPendingTransactionsDirty = true;

	// Anything left in AllTransactions is discarded
	return FilteredTransactions;
}

void FFileCache::IgnoreNewFile(const FString& Filename)
{
	auto TransactionPath = GetTransactionPath(Filename);
	if (TransactionPath.IsSet())
	{
		DirtyFiles.Remove(TransactionPath.GetValue());
		
		const FFileData FileData(IFileManager::Get().GetTimeStamp(*Filename), FMD5Hash::HashFile(*Filename));
		CompleteTransaction(FUpdateCacheTransaction(MoveTemp(TransactionPath.GetValue()), EFileAction::Added, FileData));

		bPendingTransactionsDirty = true;
	}
}

void FFileCache::IgnoreFileModification(const FString& Filename)
{
	auto TransactionPath = GetTransactionPath(Filename);
	if (TransactionPath.IsSet())
	{
		DirtyFiles.Remove(TransactionPath.GetValue());
		
		const FFileData FileData(IFileManager::Get().GetTimeStamp(*Filename), FMD5Hash::HashFile(*Filename));
		CompleteTransaction(FUpdateCacheTransaction(MoveTemp(TransactionPath.GetValue()), EFileAction::Modified, FileData));

		bPendingTransactionsDirty = true;
	}
}

void FFileCache::IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename)
{
	auto SrcTransactionPath = GetTransactionPath(SrcFilename);
	auto DstTransactionPath = GetTransactionPath(DstFilename);

	if (SrcTransactionPath.IsSet() && DstTransactionPath.IsSet())
	{
		DirtyFiles.Remove(SrcTransactionPath.GetValue());
		DirtyFiles.Remove(DstTransactionPath.GetValue());

		const FFileData FileData(IFileManager::Get().GetTimeStamp(*DstFilename), FMD5Hash::HashFile(*DstFilename));
		CompleteTransaction(FUpdateCacheTransaction(MoveTemp(SrcTransactionPath.GetValue()), MoveTemp(DstTransactionPath.GetValue()), FileData));

		bPendingTransactionsDirty = true;
	}
}

void FFileCache::IgnoreDeletedFile(const FString& Filename)
{
	auto TransactionPath = GetTransactionPath(Filename);
	if (TransactionPath.IsSet())
	{
		DirtyFiles.Remove(TransactionPath.GetValue());
		CompleteTransaction(FUpdateCacheTransaction(MoveTemp(TransactionPath.GetValue()), EFileAction::Removed));

		bPendingTransactionsDirty = true;
	}
}

void FFileCache::CompleteTransaction(FUpdateCacheTransaction&& Transaction)
{
	auto* CachedData = CachedDirectoryState.Files.Find(Transaction.Filename);
	switch (Transaction.Action)
	{
	case EFileAction::Moved:
		{
			CachedDirectoryState.Files.Remove(Transaction.MovedFromFilename);
			if (!CachedData)
			{
				CachedDirectoryState.Files.Add(Transaction.Filename, Transaction.FileData);
			}
			else
			{
				*CachedData = Transaction.FileData;
			}

			bSavedCacheDirty = true;
		}
		break;
	case EFileAction::Modified:
		if (CachedData)
		{
			// Update the timestamp
			*CachedData = Transaction.FileData;

			bSavedCacheDirty = true;
		}
		break;
	case EFileAction::Added:
		if (!CachedData)
		{
			// Add the file information to the cache
			CachedDirectoryState.Files.Emplace(Transaction.Filename, Transaction.FileData);

			bSavedCacheDirty = true;
		}
		break;
	case EFileAction::Removed:
		if (CachedData)
		{
			// Remove the file information to the cache
			CachedDirectoryState.Files.Remove(Transaction.Filename);

			bSavedCacheDirty = true;
		}
		break;

	default:
		checkf(false, TEXT("Invalid file cached transaction"));
		break;
	}
}

void FFileCache::Tick()
{
	HarvestDirtyFileHashes();
	UpdatePendingTransactions();

	/** Stage one: wait for the asynchronous directory reader to finish harvesting timestamps for the directory */
	if (DirectoryReader.IsValid())
	{
		if (!DirectoryReader->IsComplete())
		{
			return;
		}
		else
		{
			ReadStateFromAsyncReader();

			if (Config.bRequireFileHashes)
			{
				auto FilesThatNeedHashing = DirectoryReader->GetFilesThatNeedHashing();
				if (FilesThatNeedHashing.Num() > 0)
				{
					AsyncFileHasher = MakeShareable(new FAsyncFileHasher(MoveTemp(FilesThatNeedHashing)));

					AsyncTaskThread.AddTask(AsyncFileHasher);
				}
			}

			// Null out our pointer to the directory reader to indicate that we've finished
			DirectoryReader = nullptr;
		}
	}
	/** The file cache is now running, and will report changes. */
	/** Keep harvesting file hashes from the file hashing task until complete. These are much slower to gather, and only required for rename/move detection. */
	else if (AsyncFileHasher.IsValid())
	{
		double Now = FPlatformTime::Seconds();

		if (Now - LastFileHashGetTime > 5.f)
		{
			LastFileHashGetTime = Now;
			auto Hashes = AsyncFileHasher->GetCompletedData();
			if (Hashes.Num() > 0)
			{
				bSavedCacheDirty = true;
				for (const auto& Data : Hashes)
				{
					FImmutableString CachePath = *GetPathToStore(Data);

					auto* FileData = CachedDirectoryState.Files.Find(CachePath);
					if (FileData && !FileData->FileHash.IsValid())
					{
						FileData->FileHash = Data.FileHash;
					}
				}
			}

			if (AsyncFileHasher->IsComplete())
			{
				UE_LOGF(LogFileCache, Log, "Retrieving MD5 hashes for directory '%ls' took %.2fs", *Config.Directory, AsyncFileHasher->GetAge());
				AsyncFileHasher = nullptr;
			}
		}
	}
}

FString FFileCache::GetPathToStore(const FFilenameAndHash& InData) const
{
	if (Config.PathType == EPathType::Relative)
	{
		if (FPaths::IsRelative(InData.AbsoluteFilename))
		{
			return InData.AbsoluteFilename.RightChop(ConfigDirectoryStandardized.Len());
		}
		else
		{
			return InData.AbsoluteFilename.RightChop(Config.Directory.Len());
		}
	}
	else
	{
		// TODO: Convert to absolute if not already absolute
		return InData.AbsoluteFilename;
	}
}

void FFileCache::ReadStateFromAsyncReader()
{
	// We should only ever get here once. The directory reader has finished scanning, and we can now diff the results with what we had saved in the cache file.
	check(DirectoryReader->IsComplete());
	
	TOptional<FDirectoryState> LiveState = DirectoryReader->GetLiveState();
	TOptional<FDirectoryState> CachedState = DirectoryReader->GetCachedState();

	if (!CachedState.IsSet() || !Config.bDetectChangesSinceLastRun)
	{
		// If we don't have any cached data yet, just use the file data we just harvested
		CachedDirectoryState = MoveTemp(LiveState.GetValue());
		bSavedCacheDirty = true;
		return;
	}
	else
	{
		// Use the cache that we gave to the directory reader
		CachedDirectoryState = MoveTemp(CachedState.GetValue());
	}

	const FDateTime Now = FDateTime::UtcNow();
	// We already have cached data so we need to compare it with the harvested data
	// to detect additions, modifications, and removals
	for (const auto& FilenameAndData : LiveState->Files)
	{
		const FString& Filename = FilenameAndData.Key.Get();

		// If the file we've discovered was not applicable to the old cache, we can't report a change for it as we don't know if it's new or not, just add it straight to the cache.
		if (!CachedDirectoryState.Rules.IsFileApplicable(*Filename))
		{
			CachedDirectoryState.Files.Add(FilenameAndData.Key, FilenameAndData.Value);
			bSavedCacheDirty = true;
		}
		else
		{
			const auto* CachedData = CachedDirectoryState.Files.Find(Filename);
			if (!CachedData || CachedData->Timestamp != FilenameAndData.Value.Timestamp)
			{
				DirtyFiles.Add(FilenameAndData.Key, FFileData(Now, FMD5Hash()));
			}
		}
	}

	// Check for anything that doesn't exist on disk anymore
	for (auto It = CachedDirectoryState.Files.CreateIterator(); It; ++It)
	{
		const FImmutableString& Filename = It.Key();
		if (LiveState->Rules.IsFileApplicable(*Filename.Get()) && !LiveState->Files.Contains(Filename))
		{
			DirtyFiles.Add(Filename, FFileData(Now, FMD5Hash()));
		}
	}

	RescanForDirtyFileHashes();

	bPendingTransactionsDirty = true;

	// Update the applicable extensions now that we've updated the cache
	CachedDirectoryState.Rules = LiveState->Rules;
}

void FFileCache::HarvestDirtyFileHashes()
{
	if (!DirtyFileHasher.IsValid())
	{
		return;
	}
	else for (FFilenameAndHash& Data : DirtyFileHasher->GetCompletedData())
	{
		FImmutableString CachePath = *GetPathToStore(Data);

		if (auto* FileData = DirtyFiles.Find(CachePath))
		{
			FileData->FileHash = Data.FileHash;
			bPendingTransactionsDirty = true;
		}
	}

	if (DirtyFileHasher->IsComplete())
	{
		DirtyFileHasher = nullptr;
	}
}

void FFileCache::RescanForDirtyFileHashes()
{
	if (!Config.bRequireFileHashes)
	{
		return;
	}

	TArray<FFilenameAndHash> FilesThatNeedHashing;

	for (const auto& Pair : DirtyFiles)
	{
		if (!Pair.Value.FileHash.IsValid())
		{
			FilesThatNeedHashing.Emplace(GetAbsolutePath(Pair.Key.Get()));
		}
	}

	if (FilesThatNeedHashing.Num() > 0)
	{
		// Re-create the dirty file hasher with the new data that needs hashing. The old task will clean itself up if it already exists.
		DirtyFileHasher = MakeShareable(new FAsyncFileHasher(MoveTemp(FilesThatNeedHashing)));
		AsyncTaskThread.AddTask(DirtyFileHasher);
	}
}

void FFileCache::OnDirectoryChanged(const TArray<FFileChangeData>& FileChanges)
{
	// Harvest any completed data from the file hasher before we discard it
	HarvestDirtyFileHashes();

	const FDateTime Now = FDateTime::UtcNow();
	for (const auto& ThisEntry : FileChanges)
	{
		auto TransactionPath = GetTransactionPath(ThisEntry.Filename);
		if (TransactionPath.IsSet())
		{
			// Add the file that changed to the dirty files map, potentially invalidating the MD5 hash (we'll need to calculate it again)
			DirtyFiles.Add(MoveTemp(TransactionPath.GetValue()), FFileData(Now, FMD5Hash()));
			bPendingTransactionsDirty = true;
		}
	}

	RescanForDirtyFileHashes();
}

} // namespace DirectoryWatcher
