// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileCacheUtilities.h"
#include "Misc/WildcardString.h"

namespace DirectoryWatcher
{

bool MatchExtensionString(const TCHAR* Filename, const TCHAR* Extensions)
{
	const int32 StrLength = FCString::Strlen(Filename);
	const TCHAR* Ext = FCString::Strrchr(Filename, '.');
	const char QueryChar = ';';

	if (!Ext || *(++Ext) == '\0')
	{
		return false;
	}
	
	const int32 ExtLength = StrLength - UE_PTRDIFF_TO_INT32(Ext - Filename);
	const TCHAR* Search = FCString::Strchr(Extensions, QueryChar);

	while (Search && *(++Search) != '\0')
	{
		if (FCString::Strnicmp(Search, Ext, ExtLength) == 0
			&& *(Search + ExtLength) == QueryChar)
		{
			return true;
		}

		Search = FCString::Strchr(Search, QueryChar);
	}

	return false;
}

struct FWildcardRule : IMatchRule
{
	FWildcardString WildcardString;
	bool bInclude;

	virtual ~FWildcardRule(){}
	
	virtual TOptional<bool> IsFileApplicable(const TCHAR* Filename) const override
	{
		if (WildcardString.IsMatch(Filename))
		{
			return bInclude;
		}
		return TOptional<bool>();
	}

	virtual void Serialize(FArchive& Ar) override
	{
		Ar << WildcardString;
		Ar << bInclude;
	}
};

void FMatchRules::AddWildcardRule(const FWildcardString& WildcardString, bool bInclude)
{
	FMatchRule Rule;
	Rule.Type = FMatchRule::Wildcard;
	auto* Impl = new FWildcardRule;
	Impl->WildcardString = WildcardString;
	Impl->bInclude = bInclude;
	Rule.RuleImpl = MakeShareable(Impl);
	Impls.Add(Rule);

	// If there are any include patterns, we default to not matching all files
	if (bInclude)
	{
		bDefaultIncludeState = false;
	}
}

void FMatchRules::SetApplicableExtensions(const FString& InExtensions)
{
	ApplicableExtensions = InExtensions;

	// Ensure that the extension strings are of the form ;ext1;ext2;ext3;
	if (ApplicableExtensions[ApplicableExtensions.Len() - 1] != ';')
	{
		ApplicableExtensions += TEXT(";");
	}
	if (ApplicableExtensions[0] != ';')
	{
		ApplicableExtensions.InsertAt(0, ';');
	}

}

void FMatchRules::FMatchRule::Serialize(FArchive& Ar)
{
	Ar << Type;
	
	if (Ar.IsLoading())
	{
		if (Type == Wildcard)
		{
			RuleImpl = MakeShareable(new FWildcardRule);
		}
	}

	if (RuleImpl.IsValid())
	{
		RuleImpl->Serialize(Ar);
	}
}

bool FMatchRules::IsFileApplicable(const TCHAR* Filename) const
{
	if (!ApplicableExtensions.IsEmpty() && !MatchExtensionString(Filename, *ApplicableExtensions))
	{
		return false;
	}
	
	// If we have no rules, we match everything
	if (Impls.Num() == 0)
	{
		return true;
	}

	bool bApplicable = bDefaultIncludeState;

	// Otherwise we match any of the rules
	for (const auto& Rule : Impls)
	{
		if (Rule.RuleImpl.IsValid())
		{
			auto IsApplicable = Rule.RuleImpl->IsFileApplicable(Filename);
			if (IsApplicable.IsSet())
			{
				if (!IsApplicable.GetValue())
				{
					return false;
				}
				else
				{
					bApplicable = true;
				}
			}
		}
	}

	return bApplicable;
}

} // namespace DirectoryWatcher
