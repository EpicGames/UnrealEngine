// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/DirectoryWatchRequestLinux.h"
#include "HAL/FileManager.h"
#include "DirectoryWatcherPrivate.h"

#if !UE_BUILD_SHIPPING
#include "Misc/CoreMisc.h"
#endif

// To see inotify watch events:
//   TestPAL dirwatcher -LogCmds="LogDirectoryWatcher VeryVerbose"

#define EVENT_SIZE     ( sizeof(struct inotify_event) )
#define EVENT_BUF_LEN  ( 1024 * ( EVENT_SIZE + 16 ) )

#define VERBOSE_STATS  1

static bool GDumpStats = false;
static bool GDumpedError = false;
static FString GINotifyErrorMsg;

int FDirectoryWatchRequestLinux::GFileDescriptor = -1;
TMultiMap<int32, FDirectoryWatchRequestLinux::FWatchInfo> FDirectoryWatchRequestLinux::GWatchDescriptorsToWatchInfo;

static uint32 GetPathNameHash(const FString& Key)
{
	const TCHAR* Str = &Key[0];
	uint32 StrLen = sizeof(TCHAR) * Key.Len();

	return CityHash64(reinterpret_cast<const char*>(Str), StrLen);
}

FDirectoryWatchRequestLinux::FDirectoryWatchRequestLinux()
:	bWatchSubtree(false)
,	bEndWatchRequestInvoked(false)
{
}

FDirectoryWatchRequestLinux::~FDirectoryWatchRequestLinux()
{
	Shutdown();
}

void FDirectoryWatchRequestLinux::Shutdown()
{
	// Go through all watch descriptors
	for (auto MapIt = GWatchDescriptorsToWatchInfo.CreateIterator(); MapIt; ++MapIt)
	{
		// Check if this one is ours
		if (&MapIt->Value.WatchRequest == this)
		{
			int WatchDescriptor = MapIt->Key;

			// Remove this entry
			MapIt.RemoveCurrent();

			// If this was last watch descriptor for this directory, rm the inotify watch.
			if (!GWatchDescriptorsToWatchInfo.Contains(WatchDescriptor))
			{
				UE_LOGF(LogDirectoryWatcher, VeryVerbose, "- inotify_rm_watch(%d)", WatchDescriptor);

				inotify_rm_watch(GFileDescriptor, WatchDescriptor);
			}
		}
	}

	PathNameHashSet.Empty();

	if (GWatchDescriptorsToWatchInfo.IsEmpty() && (GFileDescriptor != -1))
	{
		close(GFileDescriptor);
		GFileDescriptor = -1;
	}
}

bool FDirectoryWatchRequestLinux::Init(const FString& InDirectory, uint32 Flags)
{
	checkf(IsInGameThread(), TEXT("INotify operations only support on main thread"));

	if (InDirectory.Len() == 0)
	{
		// Verify input
		return false;
	}

	Shutdown();

	// Make sure the path is absolute
	WatchDirectory = FPaths::ConvertRelativePathToFull(InDirectory);
	bWatchSubtree = (Flags & IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree) == 0;
	bEndWatchRequestInvoked = false;

	UE_LOGF(LogDirectoryWatcher, Verbose, "Adding watch for directory tree '%ls'", *WatchDirectory);

	if (GFileDescriptor == -1)
	{
		GFileDescriptor = inotify_init1(IN_NONBLOCK | IN_CLOEXEC);
		if (GFileDescriptor == -1)
		{
			if (errno == EMFILE)
			{
				SetINotifyErrorMsg(TEXT("Failed to init inotify (ran out of inotify instances)"));
			}
			else
			{
				UE_LOGF(LogDirectoryWatcher, Error, "Failed to init inotify (errno=%d, %ls)", errno, UTF8_TO_TCHAR(strerror(errno)));
			}
			return false;
		}
	}

	// Find all subdirs and add inotify watch requests
	WatchDirectoryTree(WatchDirectory, nullptr);

	return true;
}

FDelegateHandle FDirectoryWatchRequestLinux::AddDelegate(const IDirectoryWatcher::FDirectoryChanged& InDelegate, uint32 Flags)
{
	Delegates.Emplace(InDelegate, Flags);
	return Delegates.Last().Key.GetHandle();
}

bool FDirectoryWatchRequestLinux::RemoveDelegate(FDelegateHandle InHandle)
{
	return Delegates.RemoveAll([=](const FWatchDelegate& Delegate) {
		return Delegate.Key.GetHandle() == InHandle;
	}) != 0;
}

bool FDirectoryWatchRequestLinux::HasDelegates() const
{
	return Delegates.Num() > 0;
}

void FDirectoryWatchRequestLinux::EndWatchRequest()
{
	bEndWatchRequestInvoked = true;
}

void FDirectoryWatchRequestLinux::ProcessNotifications(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap)
{
	checkf(IsInGameThread(), TEXT("INotify operations only support on main thread"));

	ProcessAllINotifyChanges();
	
	// Create a list of the WatchRequests to process this tick. Do not iterate over RequestMap when calling ProcessPendingNotifications because
	// calls to Register or Unregister requests while processing delegates would modify it while we are iterating over it
	TArray<FDirectoryWatchRequestLinux*> WatchRequestsToProcessThisFrame;
	WatchRequestsToProcessThisFrame.Reserve(RequestMap.Num());
	for (auto MapIt = RequestMap.CreateConstIterator(); MapIt; ++MapIt)
	{
		WatchRequestsToProcessThisFrame.Add(MapIt.Value());
	}
	
	// Trigger any file change notification delegates
	for (FDirectoryWatchRequestLinux* WatchRequest : WatchRequestsToProcessThisFrame)
	{
		WatchRequest->ProcessPendingNotifications();
	}

	DumpINotifyErrorDetails(RequestMap);
}

void FDirectoryWatchRequestLinux::DumpStats(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap)
{
	GDumpStats = true;
	DumpINotifyErrorDetails(RequestMap);
}

void FDirectoryWatchRequestLinux::ProcessPendingNotifications()
{
	// Trigger all listening delegates with the files that have changed
	if (FileChanges.Num() > 0)
	{
		TMap<uint32, TArray<FFileChangeData>> FileChangeCache;

		for (const FWatchDelegate& Delegate : Delegates)
		{
			// Filter list of all file changes down to ones that just match this delegate's flags
			TArray<FFileChangeData>* CachedChanges = FileChangeCache.Find(Delegate.Value);

			if (CachedChanges)
			{
				Delegate.Key.Execute(*CachedChanges);
			}
			else
			{
				const bool bIncludeDirs = (Delegate.Value & IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges) != 0;
				TArray<FFileChangeData>& Changes = FileChangeCache.Add(Delegate.Value);

				for (const TPair<FFileChangeData, bool>& FileChangeData : FileChanges)
				{
					if (!FileChangeData.Value || bIncludeDirs)
					{
						Changes.Add(FileChangeData.Key);
					}
				}
				Delegate.Key.Execute(Changes);
			}
		}

		FileChanges.Empty();
	}
}

void FDirectoryWatchRequestLinux::WatchDirectoryTree(const FString & RootAbsolutePath, TArray<TPair<FFileChangeData, bool>>* FileChangesPtr)
{
	checkf(IsInGameThread(), TEXT("INotify operations only support on main thread"));

	if (bEndWatchRequestInvoked || (GFileDescriptor == -1))
	{
		return;
	}

	// If this isn't our root watch directory or under it, don't watch
	if (!RootAbsolutePath.StartsWith(WatchDirectory, ESearchCase::CaseSensitive))
	{
		return;
	}

	if (FileChangesPtr)
	{
		FileChangesPtr->Emplace(FFileChangeData(RootAbsolutePath, FFileChangeData::FCA_Added), true);
	}

	if (!bWatchSubtree && (RootAbsolutePath != WatchDirectory))
	{
		return;
	}

	UE_LOGF(LogDirectoryWatcher, VeryVerbose, "Watching tree '%ls'", *RootAbsolutePath);

	TArray<FString> AllFiles;
	if (bWatchSubtree)
	{
		IPlatformFile::GetPlatformPhysical().IterateDirectoryRecursively(*RootAbsolutePath,
			[&AllFiles, FileChangesPtr](const TCHAR* Name, bool bIsDirectory)
				{
					if (bIsDirectory)
					{
						AllFiles.Add(Name);
					}

					if (FileChangesPtr)
					{
						FileChangesPtr->Emplace(FFileChangeData(Name, FFileChangeData::FCA_Added), bIsDirectory);
					}
					return true;
				});
	}

	// Add root path
	AllFiles.Add(RootAbsolutePath);

	for (const FString& FolderName: AllFiles)
	{
		uint32 PathNameHash = GetPathNameHash(FolderName);

		// Check if we're already watching this directory
		if (!PathNameHashSet.Contains(PathNameHash))
		{
			// If we watch a directory twice, it'll return the same Watch Descriptor
			int32 NotifyFilter = IN_CREATE | IN_MOVE | IN_MODIFY | IN_DELETE | IN_ONLYDIR;
			int32 WatchDescriptor = inotify_add_watch(GFileDescriptor, TCHAR_TO_UTF8(*FolderName), NotifyFilter);

			if (WatchDescriptor == -1)
			{
				// ENOSPC: The user limit on the total number of inotify watches was reached or the kernel failed to allocate a needed resource.
				if (errno == ENOSPC)
				{
					FString ErrorMsg = FString::Printf(
						TEXT("inotify_add_watch cannot watch folder %s (Out of inotify watches)"), *FolderName);
					SetINotifyErrorMsg(ErrorMsg);
				}
				else
				{
					UE_LOGF(LogDirectoryWatcher, Warning, "inotify_add_watch cannot watch folder %ls (errno = %d, %ls)",
							*FolderName, errno, UTF8_TO_TCHAR(strerror(errno)));
				}
			}
			else
			{
				UE_LOGF(LogDirectoryWatcher, VeryVerbose, "+ Added WatchDescriptor %d for '%ls'", WatchDescriptor, *FolderName);

				// Set the inotify watch descriptor -> folder name mapping
				FWatchInfo WatchInfo{ FolderName, *this };
				GWatchDescriptorsToWatchInfo.Add(WatchDescriptor, WatchInfo);

				// Add hashed directory path
				PathNameHashSet.Add(PathNameHash);
			}
		}
	}
}

void FDirectoryWatchRequestLinux::UnwatchDirectoryTree(const FString& RootAbsolutePath)
{
	checkf(IsInGameThread(), TEXT("INotify operations only support on main thread"));

	UE_LOGF(LogDirectoryWatcher, VeryVerbose, "Unwatching tree '%ls'", *RootAbsolutePath);

	for (auto MapIt = GWatchDescriptorsToWatchInfo.CreateIterator(); MapIt; ++MapIt)
	{
		int WatchDescriptor = MapIt->Key;
		const FWatchInfo& WatchInfo = MapIt->Value;

		// Check if this one is ours
		if (&MapIt->Value.WatchRequest != this)
		{
			continue;
		}

		if (WatchInfo.FolderName.StartsWith(RootAbsolutePath, ESearchCase::CaseSensitive))
		{
			UE_LOGF(LogDirectoryWatcher, VeryVerbose, "- Removing WatchDescriptor %d for '%ls'", WatchDescriptor, *WatchInfo.FolderName);

			PathNameHashSet.Remove(GetPathNameHash(WatchInfo.FolderName));

			// Safe version of:
			//   GWatchDescriptorsToWatchInfo.Remove(WatchDescriptor);
			MapIt.RemoveCurrent();

			// If that was the last reference to this watch descriptor, remove the inotify watch
			if (!GWatchDescriptorsToWatchInfo.Contains(WatchDescriptor))
			{
				// delete the descriptor
				int RetVal = inotify_rm_watch(GFileDescriptor, WatchDescriptor);

				UE_LOGF(LogDirectoryWatcher, VeryVerbose, "- inotify_rm_watch(%d): %d", WatchDescriptor, RetVal ? errno : 0);

				// This function may be called when root path has been deleted, and inotify_rm_watch() will fail
				// with an EINVAL when removing a watch on a deleted file.
				if (RetVal == -1 && errno != EINVAL)
				{
					UE_LOGF(LogDirectoryWatcher, Warning, "inotify_rm_watch cannot remove descriptor %d for folder '%ls' (errno = %d, %ls)",
							WatchDescriptor, *WatchInfo.FolderName, errno, ANSI_TO_TCHAR(strerror(errno)));
				}
			}
		}
	}
}

static FString INotifyFlagsToStr(uint32 INotifyFlags)
{
#if UE_BUILD_SHIPPING
	return FString();
#else
	FString Ret = TEXT("[");

#define _XTAG(_x) if (INotifyFlags & _x) Ret += FString(TEXT(" ")) + TEXT(#_x)
	_XTAG(IN_ACCESS);
	_XTAG(IN_MODIFY);
	_XTAG(IN_ATTRIB);
	_XTAG(IN_CLOSE_WRITE);
	_XTAG(IN_CLOSE_NOWRITE);
	_XTAG(IN_OPEN);
	_XTAG(IN_MOVED_FROM);
	_XTAG(IN_MOVED_TO);
	_XTAG(IN_CREATE);
	_XTAG(IN_DELETE);
	_XTAG(IN_DELETE_SELF);
	_XTAG(IN_MOVE_SELF);
	_XTAG(IN_UNMOUNT);
	_XTAG(IN_Q_OVERFLOW);
	_XTAG(IN_IGNORED);
	_XTAG(IN_ISDIR);
#undef _XTAG

	Ret += TEXT(" ]");
	return Ret;
#endif
}

void FDirectoryWatchRequestLinux::ProcessNotifyChanges(const FString& FolderName, const struct inotify_event* Event)
{
	if (bEndWatchRequestInvoked)
	{
		return;
	}

	int WatchDescriptor = Event->wd;
	bool bIsDir = (Event->mask & IN_ISDIR) != 0;
	FFileChangeData::EFileChangeAction Action = FFileChangeData::FCA_Unknown;
	FString AffectedFile = FolderName / UTF8_TO_TCHAR(Event->name);

	UE_LOGF(LogDirectoryWatcher, VeryVerbose, "Event: WatchDescriptor %d, mask 0x%08x, EventPath: '%ls' Event Name: '%ls' Len: %u %ls",
		WatchDescriptor, Event->mask, *FolderName, UTF8_TO_TCHAR(Event->name), Event->len, *INotifyFlagsToStr(Event->mask));

	if ((Event->mask & IN_CREATE) || (Event->mask & IN_MOVED_TO))
	{
		// IN_CREATE: File/directory created in watched directory
		// IN_MOVED_TO: Generated for the directory containing the new filename when a file is renamed
		if (bIsDir)
		{
			// If a directory was created/moved, watch it and add changes to FileChanges.
			// Leave Action as FCA_Unknown so nothing gets added down below.
			WatchDirectoryTree(AffectedFile, &FileChanges);
		}
		else
		{
			Action = FFileChangeData::FCA_Added;
		}
	}
	else if (Event->mask & IN_MODIFY)
	{
		// IN_MODIFY: File was modified
		// If a directory was modified, we expect to get events from already watched files in it
		Action = FFileChangeData::FCA_Modified;
	}
	// Check if the file/directory itself has been deleted (IGNORED can also be sent on delete)
	else if ((Event->mask & IN_DELETE_SELF) || (Event->mask & IN_UNMOUNT))
	{
		// IN_DELETE_SELF: Watched file/directory was itself deleted.
		//   In addition, an IN_IGNORED event will subsequently be generated for the watch descriptor
		// IN_UNMOUNT: Filesystem containing watched object was unmounted.
		//   In addition, an IN_IGNORED event will subsequently be generated for the watch descriptor

		// If a directory was deleted, we expect to get events from already watched files in it

		// NOTE: This code should ever get called - we only watch directories.
		checkf(bIsDir, TEXT("Watched item was file?"));

		if (bIsDir)
		{
			UnwatchDirectoryTree(AffectedFile);
			Action = FFileChangeData::FCA_Removed;
		}
	}
	else if (Event->mask & IN_IGNORED)
	{
		// IN_IGNORED: Watch was removed explicitly (inotify_rm_watch) or
		//   automatically (file was deleted, or filesystem was unmounted).
		PathNameHashSet.Remove(GetPathNameHash(FolderName));
		GWatchDescriptorsToWatchInfo.Remove(WatchDescriptor);
	}
	else if ((Event->mask & IN_DELETE) || (Event->mask & IN_MOVED_FROM))
	{
		// IN_DELETE: File/directory deleted from watched directory
		// IN_MOVED_FROM: Generated for the directory containing the old filename when a file is renamed

		// If a directory was deleted/moved, unwatch it
		if (bIsDir)
		{
			UnwatchDirectoryTree(AffectedFile);
		}

		Action = FFileChangeData::FCA_Removed;
	}

	if (Action != FFileChangeData::FCA_Unknown)
	{
		FileChanges.Emplace(FFileChangeData(AffectedFile, Action), bIsDir);
	}
}

void FDirectoryWatchRequestLinux::ProcessAllINotifyChanges()
{
	uint8_t Buffer[EVENT_BUF_LEN] __attribute__ ((aligned(__alignof__(struct inotify_event))));

	if (GFileDescriptor == -1)
	{
		return;
	}

	// Loop while events can be read from inotify file descriptor
	for (;;)
	{
		// Read event stream
		ssize_t Len = read(GFileDescriptor, Buffer, EVENT_BUF_LEN);

		// If the non-blocking read() found no events to read, then it returns -1 with errno set to EAGAIN.
		if (Len == -1 && errno != EAGAIN)
		{
			UE_LOGF(LogDirectoryWatcher, Error, "FDirectoryWatchRequestLinux::ProcessAllINotifyChanges() read() error (errno = %d, %ls)",
				errno, ANSI_TO_TCHAR(strerror(errno)));
			break;
		}

		if (Len <= 0)
		{
			break;
		}

		// Loop over all events in the buffer
		uint8_t* Ptr = Buffer;
		while (Ptr < Buffer + Len)
		{
			const struct inotify_event* Event;

			Event = reinterpret_cast<const struct inotify_event *>(Ptr);
			Ptr += EVENT_SIZE + Event->len;

			// Skip if overflowed
			if ((Event->wd != -1) && (Event->mask & IN_Q_OVERFLOW) == 0)
			{
				TArray<FWatchInfo> WatchInfos;
				GWatchDescriptorsToWatchInfo.MultiFind(Event->wd, WatchInfos);

				for (FWatchInfo& WatchInfo : WatchInfos)
				{
					WatchInfo.WatchRequest.ProcessNotifyChanges(WatchInfo.FolderName, Event);
				}
			}
		}
	}
}

static uint32 get_inotify_procfs_value(const char *FileName)
{
	char Buf[256];
	uint32 InterfaceVal = 0;
	
	snprintf(Buf, sizeof(Buf), "/proc/sys/fs/inotify/%s", FileName);
	Buf[sizeof(Buf) - 1] = 0;
	
	FILE* FilePtr = fopen(Buf, "r");
	if (FilePtr)
	{
		if (fscanf(FilePtr, "%u", &InterfaceVal) != 1)
		{
			InterfaceVal = 0;
		}
		fclose(FilePtr);
	}

	return InterfaceVal;
}

void FDirectoryWatchRequestLinux::SetINotifyErrorMsg(const FString &ErrorMsg)
{
	if (!GINotifyErrorMsg.Len())
	{
		GINotifyErrorMsg = ErrorMsg;
	}
}

static FString GetLinkName(const char *Pathname)
{
	FString Result;
	char Filename[PATH_MAX + 1];

	ssize_t Ret = readlink(Pathname, Filename, sizeof(Filename));
	if ((Ret > 0) && (Ret < sizeof(Filename)))
	{
		Filename[Ret] = 0;
		Result = Filename;
	}
	return Result;
}

static uint32 INotifyParseFDInfoFile(const FString& Executable, int Pid, const char *d_name)
{
	uint32 INotifyCount = 0;

	FILE* FilePtr = fopen(d_name, "r");
	if (FilePtr)
	{
		char line_buf[256];

		for (;;)
		{
			if (!fgets(line_buf, sizeof(line_buf), FilePtr))
			{
				break;
			}

			if (!strncmp(line_buf, "inotify ", 8))
			{
				INotifyCount++;
			}
		}

		fclose(FilePtr);
	}

	return INotifyCount;
}

static void INotifyParseFDDir(const FString& Executable, int Pid, uint32 &INotifyCountTotal, uint32& INotifyInstancesTotal)
{
	char Buf[256];
	uint32 INotifyCount = 0;
	uint32 INotifyInstances = 0;

	snprintf(Buf, sizeof(Buf), "/proc/%d/fd", Pid);
	Buf[sizeof(Buf) - 1] = 0;

	DIR* dir_fd = opendir(Buf);
	if (dir_fd)
	{
		for (;;)
		{
			struct dirent* dp_fd = readdir(dir_fd);
			if (!dp_fd)
			{
				break;
			}

			if ((dp_fd->d_type == DT_LNK) && isdigit(dp_fd->d_name[0]))
			{
				snprintf(Buf, sizeof(Buf), "/proc/%d/fd/%s", Pid, dp_fd->d_name);
				Buf[sizeof(Buf) - 1] = 0;

				FString Filename = GetLinkName(Buf);
				if (Filename == TEXT("anon_inode:inotify"))
				{
					snprintf(Buf, sizeof(Buf), "/proc/%d/fdinfo/%s", Pid, dp_fd->d_name);
					Buf[sizeof(Buf) - 1] = 0;

					uint32 Count = INotifyParseFDInfoFile(Executable, Pid, Buf);
					if (Count)
					{
						INotifyInstances++;
						INotifyCount += Count;
					}
				}
			}
		}

		closedir(dir_fd);
	}

	if (INotifyCount)
	{
		FString ExeName = FPaths::GetCleanFilename(Executable);

#if !VERBOSE_STATS
		if (Pid == getpid())
#endif
		{
			UE_LOGF(LogDirectoryWatcher, Warning, "  %ls (pid %d) watches:%u instances:%u", *ExeName, Pid, INotifyCount, INotifyInstances);
		}

		INotifyCountTotal += INotifyCount;
		INotifyInstancesTotal += INotifyInstances;
	}
}

static void INotifyDumpProcessStats()
{
	uint32 INotifyCountTotal = 0;
	uint32 INotifyInstancesTotal = 0;

	DIR* dir_proc = opendir("/proc");
	if (dir_proc)
	{
		for (;;)
		{
			struct dirent* dp_proc = readdir(dir_proc);
			if (!dp_proc)
			{
				break;
			}

			if ((dp_proc->d_type == DT_DIR) && isdigit(dp_proc->d_name[0]))
			{
				char Buf[256];
				int32 Pid = atoi(dp_proc->d_name);

				snprintf(Buf, sizeof(Buf), "/proc/%d/exe", Pid);
				Buf[sizeof(Buf) - 1] = 0;

				FString Executable = GetLinkName(Buf);
				if (Executable.Len())
				{
					INotifyParseFDDir(Executable, Pid, INotifyCountTotal, INotifyInstancesTotal);
				}
			}
		}

		closedir(dir_proc);
	}

	UE_LOGF(LogDirectoryWatcher, Warning, "Total inotify Watches:%u Instances:%u", INotifyCountTotal, INotifyInstancesTotal);
}

void FDirectoryWatchRequestLinux::DumpINotifyErrorDetails(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap)
{
	if (!GDumpStats)
	{
		if (GDumpedError || !GINotifyErrorMsg.Len())
		{
			return;
		}
		GDumpedError = true;

		UE_LOGF(LogDirectoryWatcher, Warning, "inotify error: %ls", *GINotifyErrorMsg);
	}
	GDumpStats = false;

#if VERBOSE_STATS
	uint32 MaxQueuedEvents = get_inotify_procfs_value("max_queued_events");
	uint32 MaxUserInstances = get_inotify_procfs_value("max_user_instances");
	uint32 MaxUserWatches = get_inotify_procfs_value("max_user_watches");

	UE_LOGF(LogDirectoryWatcher, Warning, "inotify limits");
	UE_LOGF(LogDirectoryWatcher, Warning, "  max_queued_events: %u", MaxQueuedEvents);
	UE_LOGF(LogDirectoryWatcher, Warning, "  max_user_instances: %u", MaxUserInstances);
	UE_LOGF(LogDirectoryWatcher, Warning, "  max_user_watches: %u", MaxUserWatches);
#endif

	UE_LOGF(LogDirectoryWatcher, Warning, "inotify per-process stats");
	INotifyDumpProcessStats();

	uint32 Count = 0;
	UE_LOGF(LogDirectoryWatcher, Warning, "Current watch requests");

	for (auto MapIt = RequestMap.CreateConstIterator(); MapIt; ++MapIt)
	{
		uint32 DirCount = 1;
		FDirectoryWatchRequestLinux &WatchRequest = *MapIt.Value();

		if (!IPlatformFile::GetPlatformPhysical().DirectoryExists(*WatchRequest.WatchDirectory))
		{
			UE_LOGF(LogDirectoryWatcher, Warning, "  %ls: %u watches (dir does not exist)",
					*WatchRequest.WatchDirectory, WatchRequest.PathNameHashSet.Num());
		}
		else
		{
			// Get actual count of subdirectories
			if (WatchRequest.bWatchSubtree)
			{
				IPlatformFile::GetPlatformPhysical().IterateDirectoryRecursively(*WatchRequest.WatchDirectory,
						[&DirCount](const TCHAR* Name, bool bIsDirectory)
						{
						DirCount += bIsDirectory;
						return true;
						});
			}

			UE_LOGF(LogDirectoryWatcher, Warning, "  %ls: %u watches (%u total dirs)",
					*WatchRequest.WatchDirectory, WatchRequest.PathNameHashSet.Num(), DirCount);

			Count += WatchRequest.PathNameHashSet.Num();
		}
	}

	UE_LOGF(LogDirectoryWatcher, Warning, "Total UE inotify Watches:%u WatchDescriptors:%u Instances:%u ",
			Count, GWatchDescriptorsToWatchInfo.Num(), RequestMap.Num());
}

#if !UE_BUILD_SHIPPING

static bool INotifyCommandHandler(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("DumpINotifyStats")))
	{
		UE_LOGF(LogDirectoryWatcher, Warning, "Dumping inotify stats");
		GDumpStats = true;
		return true;
	}

	return false;
}

FStaticSelfRegisteringExec FDirectoryWatchRequestLinuxExecs(INotifyCommandHandler);

#endif
