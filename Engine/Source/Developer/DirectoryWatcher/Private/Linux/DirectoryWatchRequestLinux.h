// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDirectoryWatcher.h"
#include <sys/inotify.h>

class FDirectoryWatchRequestLinux
{
public:

	FDirectoryWatchRequestLinux();
	virtual ~FDirectoryWatchRequestLinux();

	/** Sets up the directory handle and request information */
	bool Init(const FString& InDirectory, uint32 Flags);

	/** Adds a delegate to get fired when the directory changes */
	FDelegateHandle AddDelegate( const IDirectoryWatcher::FDirectoryChanged& InDelegate, uint32 Flags );
	/** Removes a delegate to get fired when the directory changes */
	bool RemoveDelegate( FDelegateHandle InHandle );
	/** Returns true if this request has any delegates listening to directory changes */
	bool HasDelegates() const;
	/** Prepares the request for deletion */
	void EndWatchRequest();

	/** Call ProcessPendingNotifications on each delegate */
	static void ProcessNotifications(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap);

	/** Dump inotify stats */
	static void DumpStats(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap);

private:

	/** Triggers all pending file change notifications */
	void ProcessPendingNotifications();

	/** Adds watches for all files (and subdirectories) in a directory. */
	void WatchDirectoryTree(const FString& RootAbsolutePath, TArray<TPair<FFileChangeData, bool>>* FileChanges);

	/** Removes all watches for path */
	void UnwatchDirectoryTree(const FString& RootAbsolutePath);

	void Shutdown();

	void ProcessNotifyChanges(const FString& FolderName, const struct inotify_event* Event);

	static void ProcessAllINotifyChanges();

	static void SetINotifyErrorMsg(const FString &ErrorMsg);
	static void DumpINotifyErrorDetails(TMap<FString, FDirectoryWatchRequestLinux*>& RequestMap);

private:

	/** Whether or not watch subtree. */
	bool bWatchSubtree;

	/** EndWatchRequest called? */
	bool bEndWatchRequestInvoked;

	/** Absolute path to our root watch directory */
	FString WatchDirectory;

	/** Set of hashed directory names we're watching */
	TSet<uint32> PathNameHashSet;

	/** A delegate with its corresponding IDirectoryWatcher::WatchOptions flags */
	typedef TPair<IDirectoryWatcher::FDirectoryChanged, uint32> FWatchDelegate;
	TArray<FWatchDelegate> Delegates;

	TArray<TPair<FFileChangeData, bool>> FileChanges;

private:

	/** INotify file descriptor */
	static int GFileDescriptor;

	/** Mapping from inotify watch descriptors to watched directory names + watch request */
	struct FWatchInfo
	{
		FString FolderName;
		FDirectoryWatchRequestLinux &WatchRequest;
	};
	static TMultiMap<int32, FWatchInfo> GWatchDescriptorsToWatchInfo;
};

