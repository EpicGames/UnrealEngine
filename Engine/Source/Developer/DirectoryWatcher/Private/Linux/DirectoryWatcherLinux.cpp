// Copyright Epic Games, Inc. All Rights Reserved.

#include "Linux/DirectoryWatcherLinux.h"
#include "DirectoryWatcherPrivate.h"

FDirectoryWatcherLinux::FDirectoryWatcherLinux()
{
	NumRequests = 0;
}

FDirectoryWatcherLinux::~FDirectoryWatcherLinux()
{
	if (RequestMap.Num() != 0)
	{
		// Delete any remaining requests here. These requests are likely from modules which are still loaded at the time that this module unloads.
		for (TMap<FString, FDirectoryWatchRequestLinux*>::TConstIterator RequestIt(RequestMap); RequestIt; ++RequestIt)
		{
			if (ensure(RequestIt.Value()))
			{
				delete RequestIt.Value();
				NumRequests--;
			}
		}

		RequestMap.Empty();
	}

	if (RequestsPendingDelete.Num() != 0)
	{
		for (int32 RequestIdx = 0; RequestIdx < RequestsPendingDelete.Num(); ++RequestIdx)
		{
			delete RequestsPendingDelete[RequestIdx];
			NumRequests--;
		}
	}

	// Make sure every request that was created is destroyed
	ensure(NumRequests == 0);
}

bool FDirectoryWatcherLinux::RegisterDirectoryChangedCallback_Handle(const FString& InDirectory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags)
{
	// Make sure the path is absolute. Without this, we were missing and duplicating some directories.
	FString WatchDirectory = FPaths::ConvertRelativePathToFull(InDirectory);
	FDirectoryWatchRequestLinux** RequestPtr = RequestMap.Find(WatchDirectory);
	FDirectoryWatchRequestLinux* Request = NULL;

	if (RequestPtr)
	{
		// There should be no NULL entries in the map
		check (*RequestPtr);

		Request = *RequestPtr;
	}
	else
	{
		Request = new FDirectoryWatchRequestLinux();
		NumRequests++;

		// Begin reading directory changes
		if (!Request->Init(WatchDirectory, Flags))
		{
			UE_LOGF(LogDirectoryWatcher, Warning, "Failed to begin reading directory changes for %ls.", *WatchDirectory);
			delete Request;
			NumRequests--;
			return false;
		}

		RequestMap.Add(WatchDirectory, Request);
	}

	OutHandle = Request->AddDelegate(InDelegate, Flags);

	return true;
}


bool FDirectoryWatcherLinux::UnregisterDirectoryChangedCallback_Handle(const FString& InDirectory, FDelegateHandle InHandle)
{
	// Make sure the path is absolute
	FString WatchDirectory = FPaths::ConvertRelativePathToFull(InDirectory);
	FDirectoryWatchRequestLinux** RequestPtr = RequestMap.Find(WatchDirectory);

	if (RequestPtr)
	{
		// There should be no NULL entries in the map
		check (*RequestPtr);

		FDirectoryWatchRequestLinux* Request = *RequestPtr;

		if (Request->RemoveDelegate(InHandle))
		{
			if (!Request->HasDelegates())
			{
				// Remove from the active map and add to the pending delete list
				RequestMap.Remove(WatchDirectory);
				RequestsPendingDelete.AddUnique(Request);

				// Signal to end the watch which will mark this request for deletion
				Request->EndWatchRequest();
			}

			return true;
		}

	}

	return false;
}

void FDirectoryWatcherLinux::Tick(float DeltaSeconds)
{
	// Delete unregistered requests
	for (int32 RequestIdx = RequestsPendingDelete.Num() - 1; RequestIdx >= 0; --RequestIdx)
	{
		FDirectoryWatchRequestLinux* Request = RequestsPendingDelete[RequestIdx];
		delete Request;
		NumRequests--;
		RequestsPendingDelete.RemoveAt(RequestIdx);
	}

	FDirectoryWatchRequestLinux::ProcessNotifications(RequestMap);
}

bool FDirectoryWatcherLinux::DumpStats()
{
	FDirectoryWatchRequestLinux::DumpStats(RequestMap);
	return true;
}

