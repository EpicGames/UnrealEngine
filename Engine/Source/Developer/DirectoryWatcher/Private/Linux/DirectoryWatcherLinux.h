// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDirectoryWatcher.h"

class FDirectoryWatchRequestLinux;

class FDirectoryWatcherLinux : public IDirectoryWatcher
{
public:
	FDirectoryWatcherLinux();
	virtual ~FDirectoryWatcherLinux();

	virtual bool RegisterDirectoryChangedCallback_Handle (const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags) override;
	virtual bool UnregisterDirectoryChangedCallback_Handle(const FString& Directory, FDelegateHandle InHandle) override;
	virtual void Tick (float DeltaSeconds) override;
	virtual bool DumpStats() override;

	/** Map of directory paths to requests */
	TMap<FString, FDirectoryWatchRequestLinux*> RequestMap;
	TArray<FDirectoryWatchRequestLinux*> RequestsPendingDelete;

	/** A count of FDirectoryWatchRequestLinux created to ensure they are cleaned up on shutdown */
	int32 NumRequests;
};

typedef FDirectoryWatcherLinux FDirectoryWatcher;
