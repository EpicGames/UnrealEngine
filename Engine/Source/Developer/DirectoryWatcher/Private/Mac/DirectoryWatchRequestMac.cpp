// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryWatchRequestMac.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Mac/CocoaThread.h"

void DirectoryWatchMacCallback( ConstFSEventStreamRef StreamRef, void* WatchRequestPtr, size_t EventCount, void* EventPaths, const FSEventStreamEventFlags EventFlags[], const FSEventStreamEventId EventIDs[] )
{
	FDirectoryWatchRequestMac* WatchRequest = (FDirectoryWatchRequestMac*)WatchRequestPtr;
	check(WatchRequest);
	check(WatchRequest->EventStream == StreamRef);

	WatchRequest->ProcessChanges( EventCount, EventPaths, EventFlags);
}

// ============================================================================================================================

FDirectoryWatchRequestMac::FDirectoryWatchRequestMac()
:	bRunning(false)
,	bEndWatchRequestInvoked(false)
{
}

FDirectoryWatchRequestMac::~FDirectoryWatchRequestMac()
{
	Shutdown();
}

void FDirectoryWatchRequestMac::Shutdown( void )
{
	if( bRunning )
	{
		check(EventStream);

		FSEventStreamStop(EventStream);
		FSEventStreamInvalidate(EventStream);
		FSEventStreamRelease(EventStream);

		bRunning = false;
	}
}

bool FDirectoryWatchRequestMac::Init(const FString& InDirectory)
{
	if ( InDirectory.Len() == 0 )
	{
		// Verify input
		return false;
	}

	if( bRunning )
	{
		Shutdown();
	}

	bEndWatchRequestInvoked = false;

	// Make sure the path is absolute
	const FString FullPath = FPaths::ConvertRelativePathToFull(InDirectory);

	// Set up streaming and turn it on
	CFStringRef FullPathMac = FPlatformString::TCHARToCFString(*FullPath);
	CFArrayRef PathsToWatch = CFArrayCreate(NULL, (const void**)&FullPathMac, 1, NULL);

	CFAbsoluteTime Latency = 0.2;	// seconds

	FSEventStreamContext Context;
	Context.version = 0;
	Context.info = this;
	Context.retain = NULL;
	Context.release = NULL;
	Context.copyDescription = NULL;

	EventStream = FSEventStreamCreate( NULL,
		&DirectoryWatchMacCallback,
		&Context,
		PathsToWatch,
		kFSEventStreamEventIdSinceNow,
		Latency,
		kFSEventStreamCreateFlagUseCFTypes | kFSEventStreamCreateFlagNoDefer | kFSEventStreamCreateFlagFileEvents
	);

	CFRelease(PathsToWatch);
	CFRelease(FullPathMac);

	if( !EventStream )
	{
		return false;
	}

	FSEventStreamSetDispatchQueue( EventStream, dispatch_get_main_queue() );
	FSEventStreamStart( EventStream );

	bRunning = true;

	return true;
}

FDelegateHandle FDirectoryWatchRequestMac::AddDelegate( const IDirectoryWatcher::FDirectoryChanged& InDelegate, uint32 Flags )
{
	Delegates.Emplace(InDelegate, Flags);
	return Delegates.Last().Key.GetHandle();
}

bool FDirectoryWatchRequestMac::RemoveDelegate( FDelegateHandle InHandle )
{
	return Delegates.RemoveAll([=](const FWatchDelegate& Delegate) {
		return Delegate.Key.GetHandle() == InHandle;
	}) != 0;
}

bool FDirectoryWatchRequestMac::HasDelegates() const
{
	return Delegates.Num() > 0;
}

void FDirectoryWatchRequestMac::EndWatchRequest()
{
	bEndWatchRequestInvoked = true;
}

void FDirectoryWatchRequestMac::ProcessPendingNotifications()
{
	bool bNeedsEmpty = false;

	{
		FReadScopeLock Lock(FileChangesLock);

		// Trigger all listening delegates with the files that have changed
		if ( FileChanges.Num() > 0 )
		{
			TMap<uint32, TArray<FFileChangeData>> FileChangeCache;
			for (const FWatchDelegate& Delegate : Delegates)
			{
				// Filter list of all file changes down to ones that just match this delegate's flags
				TArray<FFileChangeData>* CachedChanges = FileChangeCache.Find(Delegate.Value);
				if (CachedChanges)
				{
					Delegate.Key.Execute(*CachedChanges);
				}
				else
				{
					const bool bIncludeDirs = (Delegate.Value & IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges) != 0;
					TArray<FFileChangeData>& Changes = FileChangeCache.Add(Delegate.Value);
					for (const TPair<FFileChangeData, bool>& FileChangeData : FileChanges)
					{
						// @todo support IgnoreChangesInSubtree
						if (!FileChangeData.Value || bIncludeDirs)
						{
							Changes.Add(FileChangeData.Key);
						}
					}
					Delegate.Key.Execute(Changes);
				}
			}
			
			bNeedsEmpty = true;
		}
	}
	if (bNeedsEmpty)
	{
		FWriteScopeLock Lock(FileChangesLock);
		FileChanges.Empty();
	}
}

void FDirectoryWatchRequestMac::ProcessChanges( size_t EventCount, void* EventPaths, const FSEventStreamEventFlags EventFlags[] )
{
	if( bEndWatchRequestInvoked )
	{
		// ignore all events
		return;
	}

	FWriteScopeLock Lock(FileChangesLock);

	CFArrayRef EventPathArray = (CFArrayRef)EventPaths;

	for( size_t EventIndex = 0; EventIndex < EventCount; ++EventIndex )
	{
		const FSEventStreamEventFlags Flags = EventFlags[EventIndex];
		if( !(Flags & kFSEventStreamEventFlagItemIsFile) && !(Flags & kFSEventStreamEventFlagItemIsDir) )
		{
			// events about symlinks don't concern us
			continue;
		}

		// Warning: some events have more than one of created, removed nd modified flag.
		// For safety, I think removed should take preference over created, and created over modified.
		FFileChangeData::EFileChangeAction Action;

		const bool bFileAdded = ( Flags & kFSEventStreamEventFlagItemCreated );
		const bool bFileRenamed = ( Flags & kFSEventStreamEventFlagItemRenamed );
		const bool bFileModified = ( Flags & kFSEventStreamEventFlagItemModified );
		const bool bFileRemoved = ( Flags & kFSEventStreamEventFlagItemRemoved );

		bool bFileNeedsChecking = false;

		// File modifications take precendent over everything, unless the file has actually been deleted. We only handle newly created files when 
		// kFSEventStreamEventFlagItemCreated is exclusively set. This flag can often be set when files have been renamed or copied over the top,
		// but we abstract this behavior and just mark it as modified.
		if ( bFileModified )
		{
			bFileNeedsChecking = true;
			Action = FFileChangeData::FCA_Modified;
		}
		else if ( bFileRenamed )
		{
			// for now, renames are abstracted as deletes/adds
			bFileNeedsChecking = true;
			Action = FFileChangeData::FCA_Added;
		}
		else if( bFileAdded )
		{
			if( bFileRemoved )
			{
				// Event indicates that file was both created and removed. To find out which of those events
				// happened later, we have to check for file's presence.
				bFileNeedsChecking = true;
			}
			Action = FFileChangeData::FCA_Added;
		}
		else if( bFileRemoved )
		{
			Action = FFileChangeData::FCA_Removed;
		}
		else
		{
			// events about inode, Finder info, owner change, extended attributes modification don't concern us
			continue;
		}

		const bool bIsDir = ( Flags & kFSEventStreamEventFlagItemIsDir ) != 0;

		const FString FilePath = UTF8_TO_TCHAR(([[(NSString*)(CFStringRef)CFArrayGetValueAtIndex(EventPathArray,EventIndex) precomposedStringWithCanonicalMapping] cStringUsingEncoding:NSUTF8StringEncoding]));
		// FileExists() only reports regular files, so use DirectoryExists() for directory events.
		// Otherwise a renamed/created directory fails the existence check and is misreported as a removal.
		if( bFileNeedsChecking )
		{
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			const bool bExists = bIsDir ? PlatformFile.DirectoryExists(*FilePath) : PlatformFile.FileExists(*FilePath);
			if( !bExists )
			{
				Action = FFileChangeData::FCA_Removed;
			}
		}

		FileChanges.Emplace(FFileChangeData(FilePath, Action), bIsDir);
	}
}
