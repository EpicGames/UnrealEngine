// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDirectoryWatcher.h"
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Misc/ScopeRWLock.h"

#include "Mac/MacSystemIncludes.h"
#include <CoreServices/CoreServices.h>

class FDirectoryWatchRequestMac
{
public:
	FDirectoryWatchRequestMac();
	virtual ~FDirectoryWatchRequestMac();

	/** Sets up the directory handle and request information */
	bool Init(const FString& InDirectory);

	/** Adds a delegate to get fired when the directory changes */
	FDelegateHandle AddDelegate( const IDirectoryWatcher::FDirectoryChanged& InDelegate, uint32 Flags );
	/** Removes a delegate to get fired when the directory changes */
	bool RemoveDelegate( FDelegateHandle InHandle );
	/** Returns true if this request has any delegates listening to directory changes */
	bool HasDelegates() const;
	/** Prepares the request for deletion */
	void EndWatchRequest();
	/** Triggers all pending file change notifications */
	void ProcessPendingNotifications();

private:

	FSEventStreamRef	EventStream;
	bool				bRunning;
	bool				bEndWatchRequestInvoked;

	/** A delegate with its corresponding IDirectoryWatcher::WatchOptions flags */
	typedef TPair<IDirectoryWatcher::FDirectoryChanged, uint32> FWatchDelegate;
	TArray<FWatchDelegate> Delegates;
	/** Each FFileChangeData tracks whether it is a directory or not */
	TArray<TPair<FFileChangeData, bool>> FileChanges;
	FRWLock FileChangesLock;

	friend void DirectoryWatchMacCallback( ConstFSEventStreamRef StreamRef, void* WatchRequestPtr, size_t EventCount, void* EventPaths, const FSEventStreamEventFlags EventFlags[], const FSEventStreamEventId EventIDs[] );

	void ProcessChanges( size_t EventCount, void* EventPaths, const FSEventStreamEventFlags EventFlags[] );
	void Shutdown( void );
};
