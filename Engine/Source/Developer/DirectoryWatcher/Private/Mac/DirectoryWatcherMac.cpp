// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/DirectoryWatcherMac.h"
#include "Mac/DirectoryWatchRequestMac.h"
#include "DirectoryWatcherPrivate.h"

FDirectoryWatcherMac::FDirectoryWatcherMac()
{
	NumRequests = 0;
}

FDirectoryWatcherMac::~FDirectoryWatcherMac()
{
	if ( RequestMap.Num() != 0 )
	{
		// Delete any remaining requests here. These requests are likely from modules which are still loaded at the time that this module unloads.
		for (TMap<FString, FDirectoryWatchRequestMac*>::TConstIterator RequestIt(RequestMap); RequestIt; ++RequestIt)
		{
			if ( ensure(RequestIt.Value()) )
			{
				delete RequestIt.Value();
				NumRequests--;
			}
		}

		RequestMap.Empty();
	}

	if ( RequestsPendingDelete.Num() != 0 )
	{
		for ( int32 RequestIdx = 0; RequestIdx < RequestsPendingDelete.Num(); ++RequestIdx )
		{
			delete RequestsPendingDelete[RequestIdx];
			NumRequests--;
		}
	}

	// Make sure every request that was created is destroyed
	ensure(NumRequests == 0);
}

bool FDirectoryWatcherMac::RegisterDirectoryChangedCallback_Handle( const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags )
{
	FDirectoryWatchRequestMac** RequestPtr = RequestMap.Find(Directory);
	FDirectoryWatchRequestMac* Request = NULL;
	
	if ( RequestPtr )
	{
		// There should be no NULL entries in the map
		check (*RequestPtr);

		Request = *RequestPtr;
	}
	else
	{
		Request = new FDirectoryWatchRequestMac();
		NumRequests++;

		// Begin reading directory changes
		if ( !Request->Init(Directory) )
		{
			UE_LOGF(LogDirectoryWatcher, Warning, "Failed to begin reading directory changes for %ls.", *Directory);
			delete Request;
			NumRequests--;
			return false;
		}

		RequestMap.Add(Directory, Request);
	}

	OutHandle = Request->AddDelegate(InDelegate, Flags);

	return true;
}

bool FDirectoryWatcherMac::UnregisterDirectoryChangedCallback_Handle( const FString& Directory, FDelegateHandle InHandle )
{
	FDirectoryWatchRequestMac** RequestPtr = RequestMap.Find(Directory);
	
	if ( RequestPtr )
	{
		// There should be no NULL entries in the map
		check (*RequestPtr);

		FDirectoryWatchRequestMac* Request = *RequestPtr;

		if ( Request->RemoveDelegate(InHandle) )
		{
			if ( !Request->HasDelegates() )
			{
				// Remove from the active map and add to the pending delete list
				RequestMap.Remove(Directory);
				RequestsPendingDelete.AddUnique(Request);

				// Signal to end the watch which will mark this request for deletion
				Request->EndWatchRequest();
			}

			return true;
		}
		
	}

	return false;
}

void FDirectoryWatcherMac::Tick( float DeltaSeconds )
{
	// Delete unregistered requests
	for ( int32 RequestIdx = RequestsPendingDelete.Num() - 1; RequestIdx >= 0; --RequestIdx )
	{
		FDirectoryWatchRequestMac* Request = RequestsPendingDelete[RequestIdx];
		delete Request;
		NumRequests--;
		RequestsPendingDelete.RemoveAt(RequestIdx);
	}

	// Create a list of the WatchRequests to process this tick. Do not iterate over RequestMap when calling ProcessPendingNotifications because
	// calls to Register or Unregister requests while processing delegates would modify it while we are iterating over it
	TArray<FDirectoryWatchRequestMac*> WatchRequestsToProcessThisFrame;
	WatchRequestsToProcessThisFrame.Reserve(RequestMap.Num());
	for (TMap<FString, FDirectoryWatchRequestMac*>::TConstIterator RequestIt(RequestMap); RequestIt; ++RequestIt)
	{
		WatchRequestsToProcessThisFrame.Add(RequestIt.Value());
	}
	
	// Trigger any file change notification delegates
	for (FDirectoryWatchRequestMac* WatchRequest : WatchRequestsToProcessThisFrame)
	{
		WatchRequest->ProcessPendingNotifications();
	}
}
