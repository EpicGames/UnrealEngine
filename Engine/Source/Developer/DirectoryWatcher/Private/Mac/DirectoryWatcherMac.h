// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectoryWatcher.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"

class FDirectoryWatchRequestMac;

class FDirectoryWatcherMac : public IDirectoryWatcher
{
public:
	FDirectoryWatcherMac();
	virtual ~FDirectoryWatcherMac();

	virtual bool RegisterDirectoryChangedCallback_Handle (const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags) override;
	virtual bool UnregisterDirectoryChangedCallback_Handle (const FString& Directory, FDelegateHandle InHandle) override;
	virtual void Tick (float DeltaSeconds) override;

	/** Map of directory paths to requests */
	TMap<FString, FDirectoryWatchRequestMac*> RequestMap;
	TArray<FDirectoryWatchRequestMac*> RequestsPendingDelete;

	/** A count of FDirectoryWatchRequestMac created to ensure they are cleaned up on shutdown */
	int32 NumRequests;
};

typedef FDirectoryWatcherMac FDirectoryWatcher;
