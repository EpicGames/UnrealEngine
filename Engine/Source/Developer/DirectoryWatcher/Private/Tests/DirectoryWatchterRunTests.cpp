// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "FileCacheUtilities.h"

/** Test MatchExtensionString */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMatchExtensionStringTest, "System.Plugins.Directory Watcher.File Cache.Extension Matching", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

namespace MatchExtensionTestHelpers
{
	FString ExpectedMessage = TEXT("Matched");
	FString NotExpectedMessage = TEXT("Did not erroneously match");

	bool Test(FMatchExtensionStringTest* Test, const TCHAR* Needle, const TCHAR* Haystack, bool bExpected)
	{
		bool bResult = DirectoryWatcher::MatchExtensionString(Needle, Haystack);
		Test->TestEqual(FString::Printf(TEXT("%s '%s' in '%s'"), bExpected ? *ExpectedMessage : *NotExpectedMessage, Needle, Haystack), bExpected, bResult);

		return bResult == bExpected;
	}
}

bool FMatchExtensionStringTest::RunTest(const FString& Parameters)
{
	bool Result = true;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("blatxt"), TEXT(";txt;"), false) && Result;  //FCString::Strrchr in MatchExtentionString returns Null if Filename has no periods
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt"), TEXT(";;"), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt"), TEXT(";"), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt"), TEXT(""), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt"), TEXT(";txt;"), true) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.text"), TEXT(";txt;"), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt1"), TEXT(";txt;"), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla."), TEXT(";bla;"), false) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.png"), TEXT(";png;txt;"), true) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("bla.txt"), TEXT(";png;txt;"), true) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("/folder.bin/bla.txt"), TEXT(";png;txt;"), true) && Result;
	Result = MatchExtensionTestHelpers::Test(this, TEXT("/folder.bin/bla"), TEXT(";png;bin;"), false) && Result;

	return Result;
}
