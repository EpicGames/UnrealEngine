// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryWatchRequestWindows.h"

#include "DirectoryWatcherPrivate.h"
#include "Misc/DateTime.h"

FDirectoryWatchRequestWindows::FDirectoryWatchRequestWindows(uint32 Flags)
{
	bPendingDelete = false;
	bEndWatchRequestInvoked = false;
	WatchStartedTimeStampHistory[0] = 0;
	WatchStartedTimeStampHistory[1] = 0;

	bWatchSubtree = (Flags & IDirectoryWatcher::WatchOptions::IgnoreChangesInSubtree) == 0;
	bool bIncludeDirectoryEvents = (Flags & IDirectoryWatcher::WatchOptions::IncludeDirectoryChanges) != 0;

	NotifyFilter = FILE_NOTIFY_CHANGE_FILE_NAME | (bIncludeDirectoryEvents? FILE_NOTIFY_CHANGE_DIR_NAME : 0) | FILE_NOTIFY_CHANGE_LAST_WRITE | FILE_NOTIFY_CHANGE_CREATION;

	DirectoryHandle = INVALID_HANDLE_VALUE;

	constexpr int32 InitialMaxChanges = 16384;
	SetBufferByChangeCount(InitialMaxChanges);

	FMemory::Memzero(&Overlapped, sizeof(Overlapped));

	Overlapped.hEvent = this;
}

FDirectoryWatchRequestWindows::~FDirectoryWatchRequestWindows()
{
	if ( DirectoryHandle != INVALID_HANDLE_VALUE )
	{
		::CloseHandle(DirectoryHandle);
		DirectoryHandle = INVALID_HANDLE_VALUE;
		bBufferInUse = false;
	}
}

void FDirectoryWatchRequestWindows::SetBufferByChangeCount(int32 MaxChanges)
{
	checkf(!bBufferInUse, TEXT("Reallocating the buffer while it is referenced in a pending ReadDirectoryChangesW call is invalid and will likely cause a crash."));
	BufferLength = sizeof(FILE_NOTIFY_INFORMATION) * MaxChanges;
	Buffer.Reset(reinterpret_cast<uint8*>(FMemory::Malloc(BufferLength, alignof(DWORD))));
	BackBuffer.Reset(reinterpret_cast<uint8*>(FMemory::Malloc(BufferLength, alignof(DWORD))));
	FMemory::Memzero(Buffer.Get(), BufferLength);
	FMemory::Memzero(BackBuffer.Get(), BufferLength);
}

void FDirectoryWatchRequestWindows::SetBufferBySize(int32 BufferSize)
{
	int32 MaxChanges = BufferSize / sizeof(FILE_NOTIFY_INFORMATION); // Arbitrarily we choose to round down
	SetBufferByChangeCount(MaxChanges);
}

bool FDirectoryWatchRequestWindows::Init(const FString& InDirectory)
{
	check(Buffer);

	if ( InDirectory.Len() == 0 )
	{
		// Verify input
		return false;
	}

	Directory = InDirectory;

	if ( DirectoryHandle != INVALID_HANDLE_VALUE )
	{
		// If we already had a handle for any reason, close the old handle
		::CloseHandle(DirectoryHandle);
	}

	// Make sure the path is absolute
	const FString FullPath = FPaths::ConvertRelativePathToFull(Directory);

	// Get a handle to the directory with FILE_FLAG_BACKUP_SEMANTICS as per remarks for ReadDirectoryChanges on MSDN
	DirectoryHandle = ::CreateFile(
		*FullPath,
		FILE_LIST_DIRECTORY,
		FILE_SHARE_READ | FILE_SHARE_WRITE | FILE_SHARE_DELETE,
		NULL,
		OPEN_EXISTING,
		FILE_FLAG_BACKUP_SEMANTICS | FILE_FLAG_OVERLAPPED,
		NULL
		);

	if ( DirectoryHandle == INVALID_HANDLE_VALUE )
	{
		const DWORD ErrorCode = ::GetLastError();
		// Failed to obtain a handle to this directory
		UE_LOGF(LogDirectoryWatcher, Display, "CreateFile failed for '%ls'. GetLastError code [%d]", *FullPath, ErrorCode);
		return false;
	}

	const bool bSuccess = !!::ReadDirectoryChangesW(
		DirectoryHandle,
		Buffer.Get(),
		BufferLength,
		bWatchSubtree,
		NotifyFilter,
		NULL,
		&Overlapped,
		&FDirectoryWatchRequestWindows::ChangeNotification);
	
	if ( !bSuccess  )
	{
		const DWORD ErrorCode = ::GetLastError();
		UE_LOGF(LogDirectoryWatcher, Display, "Initial ReadDirectoryChangesW failed for '%ls'. GetLastError code [%d]", *Directory, ErrorCode);
		::CloseHandle(DirectoryHandle);
		DirectoryHandle = INVALID_HANDLE_VALUE;
		return false;
	}
	WatchStartedTimeStampHistory[0] = FDateTime::UtcNow().ToUnixTimestamp();
	WatchStartedTimeStampHistory[1] = WatchStartedTimeStampHistory[0];
	bBufferInUse = true;

	return true;
}

FDelegateHandle FDirectoryWatchRequestWindows::AddDelegate( const IDirectoryWatcher::FDirectoryChanged& InDelegate )
{
	Delegates.Add(InDelegate);
	return Delegates.Last().GetHandle();
}

bool FDirectoryWatchRequestWindows::RemoveDelegate( FDelegateHandle InHandle )
{
	return Delegates.RemoveAll([=](const IDirectoryWatcher::FDirectoryChanged& Delegate) {
		return Delegate.GetHandle() == InHandle;
	}) != 0;
}

bool FDirectoryWatchRequestWindows::HasDelegates() const
{
	return Delegates.Num() > 0;
}

HANDLE FDirectoryWatchRequestWindows::GetDirectoryHandle() const
{
	return DirectoryHandle;
}

void FDirectoryWatchRequestWindows::EndWatchRequest()
{
	if ( !bEndWatchRequestInvoked && !bPendingDelete )
	{
		if ( DirectoryHandle != INVALID_HANDLE_VALUE )
		{
			CancelIoEx(DirectoryHandle, &Overlapped);
			bBufferInUse = false;
			// Clear the handle so we don't setup any more requests, and wait for the operation to finish
			HANDLE TempDirectoryHandle = DirectoryHandle;
			DirectoryHandle = INVALID_HANDLE_VALUE;
			WaitForSingleObjectEx(TempDirectoryHandle, 1000, true);
			
			::CloseHandle(TempDirectoryHandle);
		}
		else
		{
			// The directory handle was never opened
			bPendingDelete = true;
		}

		// Only allow this to be invoked once
		bEndWatchRequestInvoked = true;
	}
}

void FDirectoryWatchRequestWindows::ProcessPendingNotifications()
{
	// Trigger all listening delegates with the files that have changed
	if ( FileChanges.Num() > 0 )
	{
		for (int32 DelegateIdx = 0; DelegateIdx < Delegates.Num(); ++DelegateIdx)
		{
			if (Delegates[DelegateIdx].IsBound())
			{
				Delegates[DelegateIdx].Execute(FileChanges);
			}
			else
			{
				Delegates.RemoveAt(DelegateIdx--);
			}
		}

		FileChanges.Empty();
	}
}

void FDirectoryWatchRequestWindows::ProcessChange(uint32 Error, uint32 NumBytes)
{
	bBufferInUse = false; // Buffer reallocations are allowed in this handling code before we resubmit the watch
	auto CloseHandleAndMarkForDelete = [this]()
	{
		::CloseHandle(DirectoryHandle);
		DirectoryHandle = INVALID_HANDLE_VALUE;
		bPendingDelete = true;
	};

	if (Error == 0 && NumBytes == 0)
	{
		DWORD UnusedNumberOfBytes;
		GetOverlappedResult(DirectoryHandle, &Overlapped, &UnusedNumberOfBytes, 0);
		Error = ::GetLastError();
	}

	if (Error == ERROR_OPERATION_ABORTED) 
	{
		// The operation was aborted, likely due to EndWatchRequest canceling it.
		// Mark the request for delete so it can be cleaned up next tick.
		bPendingDelete = true;
		UE_CLOGF(!IsEngineExitRequested(), LogDirectoryWatcher, Log, "A directory notification for '%ls' was aborted.", *Directory);
		return;
	}
	bool bValidNotification = Error != ERROR_IO_INCOMPLETE && NumBytes > 0;
	bool bIsRescan = false;
	int64 RescanReportTimestamp = 0;
	if (bValidNotification)
	{
		// Swap the pointer to the backbuffer so we can start a new read as soon as possible
		Swap(Buffer, BackBuffer);
		check(Buffer && BackBuffer);
	}
	else if (Error == ERROR_INVALID_PARAMETER)
	{
		// ReadDirectoryChangesW fails with ERROR_INVALID_PARAMETER when the buffer length is greater than 64 KB and the application is
		// monitoring a directory over the network. This is due to a packet size limitation with the underlying file sharing protocols.
		constexpr int32 MaxAllowedSize = 64 * 1024;
		if (BufferLength > MaxAllowedSize)
		{
			// This error is expected, and is sent before we fail to read changes, so do not log it
			SetBufferBySize(MaxAllowedSize);
		}
		else
		{
			UE_LOGF(LogDirectoryWatcher, Display, "A directory notification failed for '%ls' with ERROR_INVALID_PARAMETER. Attempting another request...", *Directory);
		}
	}
	else if (Error == ERROR_ACCESS_DENIED)
	{
		CloseHandleAndMarkForDelete();
		UE_LOGF(LogDirectoryWatcher, Display, "A directory notification failed for '%ls' because it could not be accessed. Aborting watch request...", *Directory);
		return;
	}
	else if (Error == ERROR_NOTIFY_ENUM_DIR)
	{
		bValidNotification = true;
		bIsRescan = true;
		RescanReportTimestamp = WatchStartedTimeStampHistory[1];
	}
	else if (Error != ERROR_SUCCESS)
	{
		UE_LOGF(LogDirectoryWatcher, Display, "A directory notification failed for '%ls' with error code [%d]. Attemping another request...", *Directory, Error);
	}
	else
	{
		UE_LOGF(LogDirectoryWatcher, Display, "A directory notification failed for '%ls' due to buffer overflow. Attemping another request...", *Directory);
	}

	// Start up another read
	const bool bSuccess = !!::ReadDirectoryChangesW(
		DirectoryHandle,
		Buffer.Get(),
		BufferLength,
		bWatchSubtree,
		NotifyFilter,
		NULL,
		&Overlapped,
		&FDirectoryWatchRequestWindows::ChangeNotification);

	if ( !bSuccess  )
	{
		const DWORD ErrorCode = ::GetLastError();
		UE_LOGF(LogDirectoryWatcher, Display, "Refresh of ReadDirectoryChangesW failed. GetLastError code [%d] Handle [%p], Path [%ls]. Aborting watch request...",
			ErrorCode, DirectoryHandle, *Directory);
		CloseHandleAndMarkForDelete();
		return;
	}
	WatchStartedTimeStampHistory[1] = WatchStartedTimeStampHistory[0];
	WatchStartedTimeStampHistory[0] = FDateTime::UtcNow().ToUnixTimestamp();
	bBufferInUse = true;

	// No need to process the change if we can not execute any delegates
	if ( !HasDelegates() || !bValidNotification )
	{
		return;
	}

	if (!bIsRescan)
	{
		// Process the change
		uint8* InfoBase = BackBuffer.Get();
		do
		{
			FILE_NOTIFY_INFORMATION* NotifyInfo = (FILE_NOTIFY_INFORMATION*)InfoBase;

			// Copy the WCHAR out of the NotifyInfo so we can put a NULL terminator on it and convert it to a FString
			FString LeafFilename;
			{
				// The Memcpy below assumes that WCHAR and TCHAR are equivalent (which they should be on Windows)
				static_assert(sizeof(WCHAR) == sizeof(TCHAR), "WCHAR is assumed to be the same size as TCHAR on Windows!");

				const int32 LeafFilenameLen = NotifyInfo->FileNameLength / sizeof(WCHAR);
				LeafFilename.GetCharArray().AddZeroed(LeafFilenameLen + 1);
				FMemory::Memcpy(LeafFilename.GetCharArray().GetData(), NotifyInfo->FileName, NotifyInfo->FileNameLength);
			}

			FFileChangeData::EFileChangeAction Action;
			switch(NotifyInfo->Action)
			{
				case FILE_ACTION_ADDED:
				case FILE_ACTION_RENAMED_NEW_NAME:
					Action = FFileChangeData::FCA_Added;
					break;

				case FILE_ACTION_REMOVED:
				case FILE_ACTION_RENAMED_OLD_NAME:
					Action = FFileChangeData::FCA_Removed;
					break;

				case FILE_ACTION_MODIFIED:
					Action = FFileChangeData::FCA_Modified;
					break;

				default:
					Action = FFileChangeData::FCA_Unknown;
					break;
			}
			FileChanges.Emplace(Directory / LeafFilename, Action);

			// If there is not another entry, break the loop
			if ( NotifyInfo->NextEntryOffset == 0 )
			{
				break;
			}

			// Adjust the offset and update the NotifyInfo pointer
			InfoBase = InfoBase + NotifyInfo->NextEntryOffset;
		}
		while(true);
	}
	else
	{
		FFileChangeData& ChangeData = FileChanges.Emplace_GetRef(Directory, FFileChangeData::FCA_RescanRequired);
		ChangeData.TimeStamp = RescanReportTimestamp;
	}
}

void FDirectoryWatchRequestWindows::ChangeNotification(::DWORD Error, ::DWORD NumBytes, LPOVERLAPPED InOverlapped)
{
	FDirectoryWatchRequestWindows* Request = (FDirectoryWatchRequestWindows*)InOverlapped->hEvent;

	check(Request);
	Request->ProcessChange((uint32)Error, (uint32)NumBytes);
}
