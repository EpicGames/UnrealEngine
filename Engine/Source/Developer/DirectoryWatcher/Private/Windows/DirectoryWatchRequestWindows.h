// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "IDirectoryWatcher.h"
#include "HAL/UnrealMemory.h"
#include "Templates/UniquePtr.h"
#include "Windows/WindowsHWrapper.h"

class FDirectoryWatchRequestWindows
{
public:
	FDirectoryWatchRequestWindows(uint32 Flags);
	virtual ~FDirectoryWatchRequestWindows();

	/** Sets up the directory handle and request information */
	bool Init(const FString& InDirectory);

	/** Adds a delegate to get fired when the directory changes */
	FDelegateHandle AddDelegate( const IDirectoryWatcher::FDirectoryChanged& InDelegate );
	/** Removes a delegate to get fired when the directory changes */
	bool RemoveDelegate( FDelegateHandle InHandle );
	/** Returns true if this request has any delegates listening to directory changes */
	bool HasDelegates() const;
	/** Returns the file handle for the directory that is being watched */
	HANDLE GetDirectoryHandle() const;
	/** Closes the system resources and prepares the request for deletion */
	void EndWatchRequest();
	/** True if system resources have been closed and the request is ready for deletion */
	bool IsPendingDelete() const { return bPendingDelete; }
	/** Triggers all pending file change notifications */
	void ProcessPendingNotifications();

private:
	/** Non-static handler for an OS notification of a directory change */
	void ProcessChange(uint32 Error, uint32 NumBytes);
	void SetBufferByChangeCount(int32 MaxChanges);
	void SetBufferBySize(int32 BufferSize);

	/** Static Handler for an OS notification of a directory change */
	static void CALLBACK ChangeNotification(::DWORD Error, ::DWORD NumBytes, LPOVERLAPPED InOverlapped);

	FString Directory;
	HANDLE DirectoryHandle;
	int64 WatchStartedTimeStampHistory[2];
	uint32 NotifyFilter;
	uint32 BufferLength = 0;
	struct FDeleterFree
	{
		void operator()(uint8* Ptr) const
		{
			if (Ptr) { FMemory::Free(Ptr); }
		}
	};
	TUniquePtr<uint8, FDeleterFree> Buffer;
	TUniquePtr<uint8, FDeleterFree> BackBuffer;
	OVERLAPPED Overlapped;

	bool bPendingDelete;
	bool bEndWatchRequestInvoked;
	bool bBufferInUse = false;
	bool bWatchSubtree;

	TArray<IDirectoryWatcher::FDirectoryChanged> Delegates;
	TArray<FFileChangeData> FileChanges;
};
