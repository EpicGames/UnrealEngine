// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectoryWatcherWindows.h"
#include "DirectoryWatcherPrivate.h"

FDirectoryWatcherWindows::FDirectoryWatcherWindows()
{
	NumRequests = 0;
}

FDirectoryWatcherWindows::~FDirectoryWatcherWindows()
{
	if ( RequestMap.Num() != 0 )
	{
		// Delete any remaining requests here. These requests are likely from modules which are still loaded at the time that this module unloads.
		for (TMap<FDirectoryWithFlags, FDirectoryWatchRequestWindows*>::TConstIterator RequestIt(RequestMap); RequestIt; ++RequestIt)
		{
			if ( ensure(RequestIt.Value()) )
			{
				// make sure we end the watch request, as we may get a callback if a request is in flight
				RequestIt.Value()->EndWatchRequest();
				delete RequestIt.Value();
				NumRequests--;
			}
		}

		RequestMap.Empty();
	}

	if ( RequestsPendingDelete.Num() != 0 )
	{
		for ( int32 RequestIdx = 0; RequestIdx < RequestsPendingDelete.Num(); ++RequestIdx )
		{
			delete RequestsPendingDelete[RequestIdx];
			NumRequests--;
		}
	}

	// Make sure every request that was created is destroyed
	ensure(NumRequests == 0);
}

bool FDirectoryWatcherWindows::RegisterDirectoryChangedCallback_Handle( const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& Handle, uint32 Flags )
{
	const FString DirectoryFullPath = FPaths::ConvertRelativePathToFull(Directory);
	const FDirectoryWithFlags DirectoryKey(DirectoryFullPath, Flags);
	FDirectoryWatchRequestWindows** RequestPtr = RequestMap.Find(DirectoryKey);
	FDirectoryWatchRequestWindows* Request = NULL;
	
	if ( RequestPtr )
	{
		// There should be no NULL entries in the map
		check (*RequestPtr);

		Request = *RequestPtr;
	}
	else
	{
		Request = new FDirectoryWatchRequestWindows(Flags);
		NumRequests++;

		// Begin reading directory changes
		if ( !Request->Init(DirectoryFullPath) )
		{
			uint32 Error = GetLastError();
			TCHAR ErrorMsg[1024];
			FPlatformMisc::GetSystemErrorMessage(ErrorMsg, 1024, Error);
			UE_LOGF(LogDirectoryWatcher, Warning, "Failed to begin reading directory changes for %ls. Error: %ls (0x%08x)", *DirectoryFullPath, ErrorMsg, Error);

			delete Request;
			NumRequests--;
			return false;
		}

		RequestMap.Add(DirectoryKey, Request);
	}

	Handle = Request->AddDelegate(InDelegate);

	return true;
}

bool FDirectoryWatcherWindows::UnregisterDirectoryChangedCallback_Handle( const FString& Directory, FDelegateHandle InHandle )
{
	const FString DirectoryFullPath = FPaths::ConvertRelativePathToFull(Directory);
	
	for (TMap<FDirectoryWithFlags, FDirectoryWatchRequestWindows*>::TIterator RequestIt(RequestMap); RequestIt; ++RequestIt)
	{
		if (RequestIt->Key.Key == DirectoryFullPath)
		{
			// There should be no NULL entries in the map
			FDirectoryWatchRequestWindows* Watch = RequestIt->Value;
			check(Watch);

			if (Watch->RemoveDelegate(InHandle))
			{
				if (!Watch->HasDelegates())
				{
					// Remove from the active map and add to the pending delete list
					RequestIt.RemoveCurrent();
					RequestsPendingDelete.AddUnique(Watch);

					// Signal to end the watch which will mark this request for deletion
					Watch->EndWatchRequest();
				}

				return true;
			}
		}
	}

	return false;
}

void FDirectoryWatcherWindows::Tick( float DeltaSeconds )
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDirectoryWatcherWindows::Tick);
	// Iterate over all requests in the request map. Move any that have shutdown into the pending delete
	// list. If we find any that are not shutdown, poll for notifications (which call our callbacks) from the
	// OS before we process the callbacks we received.
	bool bHasPolled = false;

	// Create a list of the WatchRequests to process this tick. Do not iterate over RequestMap when calling ProcessPendingNotifications because
	// calls to Register or Unregister requests while processing delegates would modify it while we are iterating over it
	TArray<FDirectoryWatchRequestWindows*> WatchRequestsToProcessThisFrame;
	WatchRequestsToProcessThisFrame.Reserve(RequestMap.Num());

	for (TMap<FDirectoryWithFlags, FDirectoryWatchRequestWindows*>::TIterator RequestIt(RequestMap);
		RequestIt; ++RequestIt)
	{
		if ( RequestIt.Value()->IsPendingDelete() )
		{
			RequestsPendingDelete.AddUnique(RequestIt.Value());
			RequestIt.RemoveCurrent();
		}
		else
		{
			WatchRequestsToProcessThisFrame.Add(RequestIt.Value());
		}
	}

	for (FDirectoryWatchRequestWindows* WatchRequest : WatchRequestsToProcessThisFrame)
	{
		if (!bHasPolled)
		{
			// Trigger any OS notifications that are queued up in Windows's MessageQueue. To do this we need to put the
			// current thread into an alertable state, which we can do by calling SleepEx(0).
			SleepEx(0 /* TimeoutMs. 0 means poll notifies and yield to waiting threads but otherwise return immediately */,
					1 /* bAlertable: the notifications will be processed */);
			bHasPolled = true;
		}
		// Pass on the unreal notifications for the request if it received an OS notification.
		WatchRequest->ProcessPendingNotifications();
	}

	// Delete any stale or invalid requests, either the ones that we found during tick or ones that were
	// queued for deletion in between ticks.
	for (TArray<FDirectoryWatchRequestWindows*>::TIterator Iter(RequestsPendingDelete); Iter; ++Iter)
	{
		FDirectoryWatchRequestWindows* Request = *Iter;
		if (Request->IsPendingDelete())
		{
			// This request is safe to delete. Delete and remove it from the list
			delete Request;
			NumRequests--;
			Iter.RemoveCurrentSwap();
		}
	}
}
