// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDirectoryWatcher.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"

class FDirectoryWatchRequestWindows;

class FDirectoryWatcherWindows : public IDirectoryWatcher
{
private:
	/** A directory and its associated IDirectoryWatcher::WatchOptions, which represents a unique ID for a watch request */
	typedef TPair<FString, uint32> FDirectoryWithFlags;
public:
	FDirectoryWatcherWindows();
	virtual ~FDirectoryWatcherWindows();

	virtual bool RegisterDirectoryChangedCallback_Handle (const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags) override;
	virtual bool UnregisterDirectoryChangedCallback_Handle (const FString& Directory, FDelegateHandle InHandle) override;
	virtual void Tick (float DeltaSeconds) override;

	/** Map of directory paths to requests */
	TMap<FDirectoryWithFlags, FDirectoryWatchRequestWindows*> RequestMap;
	TArray<FDirectoryWatchRequestWindows*> RequestsPendingDelete;

	/** A count of FDirectoryWatchRequestWindows created to ensure they are cleaned up on shutdown */
	int32 NumRequests;
};

typedef FDirectoryWatcherWindows FDirectoryWatcher;
