// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FDirectoryWatcherProxy;
class IDirectoryWatcher;
struct FFileChangeData;

class FDirectoryWatcherModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

	/** Gets the directory watcher singleton or returns NULL if the platform does not support directory watching */
	virtual IDirectoryWatcher* Get();

	/** Register external changes that the OS file watcher couldn't detect (eg, a file changing in a UE sandbox) */
	virtual void RegisterExternalChanges(TArrayView<const FFileChangeData> FileChanges) const;

private:
	FDirectoryWatcherProxy* DirectoryWatcher;
};
