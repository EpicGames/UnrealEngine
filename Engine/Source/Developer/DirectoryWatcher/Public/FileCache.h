// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "FileCacheUtilities.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/ThreadSafeCounter.h"
#include "IDirectoryWatcher.h"
#include "Misc/AssertionMacros.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"
#include "Serialization/Archive.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

#define UE_API DIRECTORYWATCHER_API

struct FFileChangeData;
struct FGuid;

namespace DirectoryWatcher
{

/** Custom serialization version for FFileCache */
struct FFileCacheCustomVersion
{
	static UE_API const FGuid Key;
	enum Type {	Initial, IncludeFileHash, Latest = IncludeFileHash };
};

/** Structure representing specific information about a particular file */
struct FFileData
{
	/** Constructiors */
	FFileData() : Timestamp(0) {}
	FFileData(const FDateTime& InTimestamp, const FMD5Hash& InFileHash) : Timestamp(InTimestamp), FileHash(InFileHash) {}

	friend bool operator==(const FFileData& LHS, const FFileData& RHS)
	{
		return LHS.Timestamp == RHS.Timestamp && LHS.FileHash == RHS.FileHash;
	}

	friend bool operator!=(const FFileData& LHS, const FFileData& RHS)
	{
		return LHS.Timestamp != RHS.Timestamp || LHS.FileHash != RHS.FileHash;	
	}

	/** Serializer for this type */
	friend FArchive& operator<<(FArchive& Ar, FFileData& Data)
	{
		if (Ar.CustomVer(FFileCacheCustomVersion::Key) >= FFileCacheCustomVersion::Initial)
		{
			Ar << Data.Timestamp;
		}

		if (Ar.CustomVer(FFileCacheCustomVersion::Key) >= FFileCacheCustomVersion::IncludeFileHash)
		{
			Ar << Data.FileHash;
		}

		return Ar;
	}

	/** The cached timestamp of the file on disk */
	FDateTime Timestamp;

	/** The cached MD5 hash of the file on disk */
	FMD5Hash FileHash;
};

/** Structure representing the file data for a number of files in a directory */
struct FDirectoryState
{
	/** Default construction */
	FDirectoryState() {}

	/** Move construction */
	FDirectoryState(FDirectoryState&& In) : Rules(MoveTemp(In.Rules)), Files(MoveTemp(In.Files)) {}
	FDirectoryState& operator=(FDirectoryState&& In) { Swap(Rules, In.Rules); Swap(Files, In.Files); return *this; }

	/** The rules that define what this state applies to */
	FMatchRules Rules;

	/** Filename -> data map */
	TMap<FImmutableString, FFileData> Files;

	/** Serializer for this type */
	friend FArchive& operator<<(FArchive& Ar, FDirectoryState& State)
	{
		Ar.UsingCustomVersion(FFileCacheCustomVersion::Key);

		// Ignore any old versions to data to ensure that we generate a new cache
		if (Ar.CustomVer(FFileCacheCustomVersion::Key) >= FFileCacheCustomVersion::IncludeFileHash)
		{
			Ar << State.Rules;

			// Number of files
			int32 Num = State.Files.Num();
			Ar << Num;
			if (Ar.IsLoading())
			{
				State.Files.Reserve(Num);
			}

			Ar << State.Files;
		}

		return Ar;
	}
};

enum class EFileAction : uint8
{
	Added, Modified, Removed, Moved
};

/** A transaction issued by FFileCache to describe a change to the cache. The change is only committed once the transaction is returned to the cache (see FFileCache::CompleteTransaction). */
struct FUpdateCacheTransaction
{
	/** The path of the file to which this transaction relates */
	FImmutableString Filename;

	/** In the case of a moved file, this represents the path the file was moved from */
	FImmutableString MovedFromFilename;

	/** File data pertaining to this change at the time of dispatch */
	FFileData FileData;

	/** The type of action that prompted this transaction */
	EFileAction Action;

	/** Publically moveable */
	FUpdateCacheTransaction(FUpdateCacheTransaction&& In) : Filename(MoveTemp(In.Filename)), MovedFromFilename(MoveTemp(In.MovedFromFilename)), FileData(MoveTemp(In.FileData)), Action(MoveTemp(In.Action)) {}
	FUpdateCacheTransaction& operator=(FUpdateCacheTransaction&& In) { Swap(Filename, In.Filename); Swap(MovedFromFilename, In.MovedFromFilename); Swap(FileData, In.FileData); Swap(Action, In.Action); return *this; }

private:
	friend class FFileCache;

	/** Construction responsibility is held by FFileCache */
	FUpdateCacheTransaction(FImmutableString InFilename, EFileAction InAction, const FFileData& InFileData = FFileData())
		: Filename(MoveTemp(InFilename)), FileData(InFileData), Action(InAction)
	{}

	/** Construction responsibility is held by FFileCache */
	FUpdateCacheTransaction(FImmutableString InMovedFromFilename, FImmutableString InMovedToFilename, const FFileData& InFileData)
		: Filename(MoveTemp(InMovedToFilename)), MovedFromFilename(MoveTemp(InMovedFromFilename)), FileData(InFileData), Action(EFileAction::Moved)
	{}
	
	/** Not Copyable */
	FUpdateCacheTransaction(const FUpdateCacheTransaction&) = delete;
	FUpdateCacheTransaction& operator=(const FUpdateCacheTransaction&) = delete;
};

/** Enum specifying whether a path should be relative or absolute */
enum EPathType
{
	/** Paths should be cached relative to the root cache directory */
	Relative,

	/** Paths should be cached as absolute file system paths */
	Absolute
};

struct IAsyncFileCacheTask : public TSharedFromThis<IAsyncFileCacheTask, ESPMode::ThreadSafe>
{
	enum class EProgressResult
	{
		Finished, Pending
	};

	IAsyncFileCacheTask() : StartTime(FPlatformTime::Seconds()) {}
	virtual ~IAsyncFileCacheTask() = default;
	
	/** Tick this task. Only to be called on the task thread. */
	virtual EProgressResult Tick(const FTimeLimit& TimeLimit) = 0;

	/** Check whether this task is complete. Must be implemented in a thread-safe manner. */
	virtual bool IsComplete() const = 0;

	/** Get the age of this task in seconds */
	double GetAge() const { return FPlatformTime::Seconds() - StartTime; }

protected:

	/** The time this task started */
	double StartTime;
};

/** Simple struct that encapsulates a filename and its associated MD5 hash */
struct FFilenameAndHash
{
	FString AbsoluteFilename;
	FMD5Hash FileHash;

	FFilenameAndHash(){}
	FFilenameAndHash(const FString& File) : AbsoluteFilename(File) {}
};

/** Async task responsible for MD5 hashing a number of files, reporting completed hashes to the client when done */
struct FAsyncFileHasher : public IAsyncFileCacheTask
{
	/** Constructor */
	FAsyncFileHasher(TArray<FFilenameAndHash> InFilesThatNeedHashing);

	/** Return any completed filenames and their corresponding hashes */
	TArray<FFilenameAndHash> GetCompletedData();

	/** Returns true when this task has finished hashing all its files */
	virtual bool IsComplete() const override;

protected:

	/** Tick this reader (hashes as many files as possible in the time allowed). Returns progress state. */
	virtual EProgressResult Tick(const FTimeLimit& Limit) override;

	/** The array of data that we will process */
	TArray<FFilenameAndHash> Data;

	/** The number of items we have returned to the client. Only accessed from the main thread. */
	int32 NumReturned;

	/** The number of files that we have hashed on the task thread. Atomic - safe to access from any thread. */
	FThreadSafeCounter CurrentIndex;

	/** Scratch buffer used for reading in files */
	TArray<uint8> ScratchBuffer;
};

/**
 * Class responsible for 'asynchronously' scanning a folder for files and timestamps.
 * Example usage:
 *		FAsyncDirectoryReader Reader(TEXT("C:\\Path"), EPathType::Relative);
 *
 *		while(!Reader.IsComplete())
 *		{
 *			FPlatformProcess::Sleep(1);
 *			Reader.Tick(FTimedSignal(1));	// Do 1 second of work
 *		}
 *		TOptional<FDirectoryState> State = Reader.GetFinalState();
 */
struct FAsyncDirectoryReader : public IAsyncFileCacheTask
{
	/** Constructor that sets up the directory reader to the specified directory */
	FAsyncDirectoryReader(const FString& InDirectory, EPathType InPathType);

	/** Set what files are relevant to this reader. Calling this once the reader starts results in undefined behaviour. */
	void SetMatchRules(const FMatchRules& InRules)
	{
		if (LiveState.IsSet())
		{
			LiveState->Rules = InRules;
		}
	}

	/**
	 * Get the state of the directory once finished. Relinquishes the currently stored directory state to the client.
	 * Returns nothing if incomplete, or if GetLiveState() has already been called.
	 */
	TOptional<FDirectoryState> GetLiveState();

	/** Retrieve the cached state supplied to this class through UseCachedState(). */
	TOptional<FDirectoryState> GetCachedState();

	/** Retrieve the cached state supplied to this class through UseCachedState(). */
	TArray<FFilenameAndHash> GetFilesThatNeedHashing()
	{
		TArray<FFilenameAndHash> Swapped;
		Swap(FilesThatNeedHashing, Swapped);
		return Swapped;
	}

	/** Instruct the directory reader to use the specified cached state to lookup file hashes, where timestamps haven't changed */
	void UseCachedState(FDirectoryState InCachedState)
	{
		CachedState = MoveTemp(InCachedState);
	}

	/** Returns true when this directory reader has finished scanning the directory */
	virtual bool IsComplete() const override;

	/** Tick this reader (discover new directories / files). Returns progress state. */
	virtual EProgressResult Tick(const FTimeLimit& Limit) override;

private:
	/** Non-recursively scan a single directory for its contents. Adds results to Pending arrays. */
	void ScanDirectory(const FString& InDirectory);

	/** Returns version of path to store */
	FString GetPathToStore(const FString& InPath) const;

	/** Path to the root directory we want to scan */
	FString RootPath;

	/** Standardized path to the root directory we want to scan */
	FString StandardRootPath;

	/** Whether we should return relative or absolute paths */
	EPathType PathType;

	/** The currently discovered state of the directory - reset once relinquished to the client through GetLiveState */
	TOptional<FDirectoryState> LiveState;

	/** The previously cached  state of the directory, optional */
	TOptional<FDirectoryState> CachedState;

	/** An array of files that need hashing */
	TArray<FFilenameAndHash> FilesThatNeedHashing;

	/** A list of directories we have recursively found on our travels */
	TArray<FString> PendingDirectories;

	/** A list of files we have recursively found on our travels */
	TArray<FString> PendingFiles;

	/** Thread safe flag to signify when this class has finished reading */
	FThreadSafeBool bIsComplete;
};

/** Configuration structure required to construct a FFileCache */
struct FFileCacheConfig
{
	/** Enum that specifies what changes are required for a change to be reported. When combined, any valid change is reported. */
	enum EChangeDetection
	{
		/** Report modifications when the timestamp of a file changes */
		Timestamp,
		/** Report modifications when the contents of a file changes */
		FileHash,
	};

	FFileCacheConfig(FString InDirectory, FString InCacheFile)
		: Directory(InDirectory), CacheFile(InCacheFile), PathType(EPathType::Relative), bDetectChangesSinceLastRun(false)
		, ChangeDetectionBits(false, 2)
	{
		DetectMoves(true);
		ChangeDetectionBits[EChangeDetection::Timestamp] = true;
	}

	/** String specifying the directory on disk that the cache should reflect */
	FString Directory;

	/** String specifying the file that the cache should be saved to. When empty, no cache file will be maintained (thus only an in-memory cache is used) */
	FString CacheFile;

	/** List of rules which define what we will be watching */
	FMatchRules Rules;

	/** Path type to return, relative to the directory or absolute. */
	EPathType PathType;

	/** When true, changes to the directory since the cache shutdown will be detected and reported. When false, said changes will silently be applied to the serialized cache. */
	bool bDetectChangesSinceLastRun;

	/** User-specified custom change detection logic. Called when a file's timestamp has been modified, to see if we should report this change to the client */
	TFunction<TOptional<bool>(const FImmutableString&, const FFileData&)> CustomChangeLogic;

	/** Set up this cache to detect moves */
	FFileCacheConfig& DetectMoves(bool bInDetectMoves)
	{
		bDetectMoves = bInDetectMoves;
		if (bDetectMoves)
		{
			bRequireFileHashes = true;
		}
		return *this;
	}
	
	/** Set up this cache to generate MD5 hashes for its constituent files */
	FFileCacheConfig& RequireFileHashes(bool bInRequireFileHashes)
	{
		if (ensureMsgf(bInRequireFileHashes || !bDetectMoves, TEXT("Unable to disable file hashing when move detection is enabled")))
		{
			bRequireFileHashes = bInRequireFileHashes;
		}
		return *this;
	}

	/** Instruct the cache to report the specified changes to files */
	FFileCacheConfig& DetectChangesFor(EChangeDetection ChangeType, bool Value)
	{
		ChangeDetectionBits[ChangeType] = Value;
		return *this;
	}

private:

	/** True to detect moves and renames (based on file hash), false otherwise. Implies bRequireFileHashes. */
	bool bDetectMoves;

	/** When true, the cache will also calculate MD5 hashes for files. When true, an additional thread will be launched on startup to harvest unknown MD5 hashes for the directory. */
	bool bRequireFileHashes;

	/** Bitfied specifying how we will be detecting changes. See EChangeDetection. */
	TBitArray<> ChangeDetectionBits;

	friend class FFileCache;
};

/**
 * A class responsible for scanning a directory, and maintaining a cache of its state (files and timestamps).
 * Changes in the cache can be retrieved through GetOutstandingChanges(). Changes will be reported for any
 * change in the cached state even between runs of the process.
 */
class FFileCache
{
public:

	/** Construction from a config */
	UE_API FFileCache(const FFileCacheConfig& InConfig);

	FFileCache(const FFileCache&) = delete;
	FFileCache& operator=(const FFileCache&) = delete;

	/** Destructor */
	UE_API ~FFileCache();

	/** Destroy this cache. Cleans out the in-memory state and deletes the cache file, if present. */
	UE_API void Destroy();

	/** Get the absolute path of the directory this cache reflects */
	const FString& GetDirectory() const { return Config.Directory; }

	/** Check whether this file cache has finished starting up yet. Does not imply rename/move detection is fully initialized. (see MoveDetectionInitialized()) */
	UE_API bool HasStartedUp() const;

	/** Check whether this move/rename detection has been initiated or not. This can take much longer than startup, so can be checked separately */
	UE_API bool MoveDetectionInitialized() const;

	/** Attempt to locate file data pertaining to the specified filename.
	 *	@param		InFilename		The filename to find data for - either relative to the directory, or absolute, depending on Config.PathType.
	 *	@return		The current cached file data pertaining to the specified filename, or nullptr. May not be completely up to date if there are outstanding transactions.
	 */
	UE_API const FFileData* FindFileData(FImmutableString InFilename) const;

	/** Tick this FileCache */
	UE_API void Tick();

	/** Write out the cached file, if we have any changes to write */
	UE_API void WriteCache();

	/**
	 * Return a transaction to the cache for completion. Will update the cached state with the change
	 * described in the transaction, and mark the cache as needing to be saved.
	 */
	UE_API void CompleteTransaction(FUpdateCacheTransaction&& Transaction);

	/** Report an external change to the manager, such that a subsequent equal change reported by the os be ignored */
	UE_API void IgnoreNewFile(const FString& Filename);
	UE_API void IgnoreFileModification(const FString& Filename);
	UE_API void IgnoreMovedFile(const FString& SrcFilename, const FString& DstFilename);
	UE_API void IgnoreDeletedFile(const FString& Filename);

	/** Get the number of pending changes to the cache. */
	int32 GetNumDirtyFiles() const { return DirtyFiles.Num(); }
	
	/**
	 * Iterate the number of pending changes to the cache with the specified predicate. Assumes Tick has been called (and thus, PendingTransactions is up-to-date)
	 * @param InIter		A predicate to match files against (returning true to continue iteration, false to break), based on:
								1) the pending transaction
								2) the time the change was detected
	*/
	UE_API void IterateOutstandingChanges(TFunctionRef<bool(const FUpdateCacheTransaction&, const FDateTime&)> InPredicate) const;

	/** Get pending changes to the cache. Transactions must be returned to CompleteTransaction to update the cache.
	 *  Filter predicate receives a transaction and the time the change was reported. */
	UE_API TArray<FUpdateCacheTransaction> FilterOutstandingChanges(TFunctionRef<bool(const FUpdateCacheTransaction&, const FDateTime&)> InPredicate);
	UE_API TArray<FUpdateCacheTransaction> GetOutstandingChanges();

private:

	/** Called when the directory we are monitoring has been changed in some way */
	UE_API void OnDirectoryChanged(const TArray<FFileChangeData>& FileChanges);

	/** Diff the specified set of dirty files (absolute paths, or relative to the monitor directory), adding transactions to the specified array if necessary.
	 *	Optionally takes a directory state from which we can retrieve current file system state, without having to ask the FS directly.
	 *  Optionally exclude files that have changed since the specified threshold, to ensure that related events get grouped together correctly.
	 */
	UE_API void DiffDirtyFiles(TMap<FImmutableString, FFileData>& InDirtyFiles, TArray<FUpdateCacheTransaction>& OutTransactions, const FDirectoryState* InFileSystemState = nullptr) const;

	/** Detect a rename for the specified file */
	UE_API void DetectRename();

	/** Get the absolute path from a transaction filename */
	UE_API FString GetAbsolutePath(const FString& InTransactionPath) const;

	/** Get the transaction path from an absolute filename. Returns nothing if the absolute path is invalid (not under the correct folder, or not applicable) */
	UE_API TOptional<FString> GetTransactionPath(const FString& InAbsolutePath) const;

	/** Unbind the watcher from the directory. Called on destruction. */
	UE_API void UnbindWatcher();

	/** Read the cache file data and return the contents */
	UE_API TOptional<FDirectoryState> ReadCache() const;

	/** Called when the initial async reader has finished harvesting file system timestamps */
	UE_API void ReadStateFromAsyncReader();

	/** Called to harvest any file hashes that have been generated by the DirtyFileHasher thread task */
	UE_API void HarvestDirtyFileHashes();
	UE_API void RescanForDirtyFileHashes();

	/** Update our cache of pending transactions (to prevent diffing against the directory unnecessarily) */
	UE_API void UpdatePendingTransactions();

	/** Returns version of path to store */
	UE_API FString GetPathToStore(const FFilenameAndHash& InData) const;

private:

	/** Configuration settings applied on construction */
	FFileCacheConfig Config;

	/** Asynchronous directory reader responsible for gathering all file/timestamp information recursively from our cache directory */
	TSharedPtr<FAsyncDirectoryReader, ESPMode::ThreadSafe> DirectoryReader;
	/** Asynchronous task used to harvest the MD5 hashes of a set of filenames */
	TSharedPtr<FAsyncFileHasher, ESPMode::ThreadSafe> AsyncFileHasher;
	/** Asynchronous task used to harvest the MD5 hashes of a set of recently changed filenames */
	TSharedPtr<FAsyncFileHasher, ESPMode::ThreadSafe> DirtyFileHasher;

	/** A map of dirty files that we will use to report changes to the user. Timestamp value of FFileData here pertains to the *time of the change* not the timestamp of the file. */
	TMap<FImmutableString, FFileData> DirtyFiles;

	/** Our in-memory view of the cached directory state. */
	FDirectoryState CachedDirectoryState;

	/** Handle to the directory watcher delegate so we can delete it properly */
	FDelegateHandle WatcherDelegate;

	/** True when the cached state we have in memory is more up to date than the serialized file. Enables WriteCache() when true. */
	bool bSavedCacheDirty;

	/** The time we last retrieved file hashes from the thread */
	double LastFileHashGetTime;

	/** List of cached pending transactions */
	bool bPendingTransactionsDirty;
	TArray<FUpdateCacheTransaction> PendingTransactions;

	FString ConfigDirectoryStandardized;
};

} // namespace DirectoryWatcher

#undef UE_API
