// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class FWildcardString;

namespace DirectoryWatcher
{

/** Helper method to determine whether filename matches expected extensions */
bool MatchExtensionString(const TCHAR* Filename, const TCHAR* Extensions);

/** An immutable string with a cached CRC for efficient comparison with other strings */
struct FImmutableString
{
	/** Constructible from a raw string */
	FImmutableString(FString InString = FString()) : String(MoveTemp(InString)), CachedHash(0) {}
	FImmutableString& operator=(FString InString) { String = MoveTemp(InString); CachedHash = 0; return *this; }

	FImmutableString(const TCHAR* InString) : String(InString), CachedHash(0) {}
	FImmutableString& operator=(const TCHAR* InString) { String = InString; CachedHash = 0; return *this; }

	FImmutableString(const FImmutableString&) = default;
	FImmutableString& operator=(const FImmutableString&) = default;

	/** Move construction/assignment */
	FImmutableString(FImmutableString&& In) : String(MoveTemp(In.String)), CachedHash(MoveTemp(In.CachedHash)) {}
	FImmutableString& operator=(FImmutableString&& In) { Swap(String, In.String); Swap(CachedHash, In.CachedHash); return *this; }

	/** Check for equality with another immutable string (case-sensitive) */
	friend bool operator==(const FImmutableString& A, const FImmutableString& B)
	{
		return GetTypeHash(A) == GetTypeHash(B) && A.String == B.String;
	}

	/** Calculate the type hash for this string */
	friend uint32 GetTypeHash(const FImmutableString& In)
	{
		if (In.CachedHash == 0 && In.String.Len() != 0)
		{
			In.CachedHash = GetTypeHash(In.String);
		}
		return In.CachedHash;
	}

	/** Get the underlying string */
	const FString& Get() const { return String; }

	/** Serialise this string */
	friend FArchive& operator<<(FArchive& Ar, FImmutableString& InString)
	{
		Ar << InString.String;
		if (Ar.IsSaving())
		{
			GetTypeHash(InString);	
		}
		
		Ar << InString.CachedHash;

		return Ar;
	}

private:
	/** The string itself. */
	FString String;

	/** The cached hash of our string. 0 until compared with another string. */
	mutable uint32 CachedHash;
};

/** A time limit that counts down from the time of construction, until it hits a given delay */
struct FTimeLimit
{
	/** Constructor specifying that we should never bail out early */
	FTimeLimit() : Delay(-1) { Reset(); }

	/** Constructor specifying not to run over the specified number of seconds */
	FTimeLimit(float NumSeconds) : Delay(NumSeconds) { Reset(); }

	/** Return true if this time limit has been set up, false otherwise */
	bool IsValid() const { return Delay != -1; }
	
	/** Check whether we have exceeded the time limit */
	bool Exceeded() const { return IsValid() && FPlatformTime::Seconds() >= StartTime + Delay; }

	/** Reset the time limit to start timing again from the current time */
	void Reset() { StartTime = FPlatformTime::Seconds(); }

private:

	/** The delay specified by the user */
	float Delay;

	/** The time we started */
	double StartTime;
};

/** A rule that checks whether a file is applicable or not */
struct IMatchRule
{
	/** Serialize this rule */
	virtual void Serialize(FArchive& Ar) = 0;

	/** Test to see if a file is applicable based on this rule. Returns true if so, false of not, or empty if the file doesn't match this rule. */
	virtual TOptional<bool> IsFileApplicable(const TCHAR* Filename) const = 0;
};

/** A set of rules that specifies what files we'll be watching */
struct FMatchRules
{
	FMatchRules() : bDefaultIncludeState(true) {}

	/** Specify a wildcard match to include or exclude */
	DIRECTORYWATCHER_API void AddWildcardRule(const FWildcardString& WildcardString, bool bInclude);

	/** Specify a set of applicable extensions, ; separated. Provided as an optimization to early-out on files we're not interested in. */
	DIRECTORYWATCHER_API void SetApplicableExtensions(const FString& InExtensions);

	/** Check whether the specified file is applicable based on these rules or not */
	DIRECTORYWATCHER_API bool IsFileApplicable(const TCHAR* Filename) const;

	friend void operator<<(FArchive& Ar, FMatchRules& Rules)
	{
		Ar << Rules.bDefaultIncludeState;
		Ar << Rules.ApplicableExtensions;
		Ar << Rules.Impls;
	}

private:

	/** Implementation of a match rule, wrapping up its type, and implementation */
	struct FMatchRule
	{
		enum EType { Wildcard };
		/** The type of this rule */
		TEnumAsByte<EType> Type;
		/** The runtime implementation */
		TSharedPtr<IMatchRule> RuleImpl;

	private:
		void Serialize(FArchive& Ar);
		friend void operator<<(FArchive& Ar, FMatchRule& Rule)
		{
			Rule.Serialize(Ar);
		}
	};

	/** Optimization to ignore files that don't match the given set of extensions. Expected to be of the form ;ext1;ext2;ext3; */
	FString ApplicableExtensions;

	/** Array of implementations */
	TArray<FMatchRule> Impls;

	/** Default inclusion state. True when only exclude rules exist, false if there are any Include rules */
	bool bDefaultIncludeState;
};

} // namespace DirectoryWatcher
