// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"

struct FFileChangeData
{
	enum EFileChangeAction
	{
		FCA_Unknown,
		FCA_Added,
		FCA_Modified,
		FCA_Removed,
		FCA_RescanRequired,
	};

	FFileChangeData(const FString& InFilename, EFileChangeAction InAction)
		: Filename(InFilename)
		, Action(InAction)
	{
		FPaths::MakeStandardFilename(Filename);
	}

	/**
	 * If the Action references a specific file, the name of the file.
	 * Applies To: FCA_Added, FCA_Modified, FCA_Removed, FCA_RescanRequired.
	 * For all other actions value will be emptystring.
	 */
	FString Filename;
	/**
	 * If the Action references a timestamp, the timezone UTC UnixTimeStamp (e.g. FDateTime::ToUnixTimeStamp) of the Action.
	 * Applies to: FCA_RescanRequired.
	 * For all other actions value will be 0.
	 */
	int64 TimeStamp = 0;
	/** The reported Action. */
	EFileChangeAction Action = EFileChangeAction::FCA_Unknown;
};


/** The public interface for the directory watcher singleton. */
class IDirectoryWatcher
{
public:

	/** Options for a single watch (can be combined) */
	enum WatchOptions : uint32
	{
		/** Whether to include notifications for changes to actual directories (such as directories being created or removed). */
		IncludeDirectoryChanges			=		(1<<0),

		/** Whether changes in subdirectories need to be reported. */
		IgnoreChangesInSubtree			=		(1<<1),
	};

	/** A delegate to report directory changes */
	DECLARE_DELEGATE_OneParam(FDirectoryChanged, const TArray<struct FFileChangeData>& /*FileChanges*/);

	/** Virtual destructor */
	virtual ~IDirectoryWatcher() = default;

	/**
	 * Register a callback to fire when directories are changed
	 *
	 * @param	Directory to watch
	 * @param	Delegate to add to our callback list
	 * @param	The handle to the registered delegate, if the registration was successful.
	 * @param	Set of options to use when registering the delegates.
	 */
	virtual bool RegisterDirectoryChangedCallback_Handle (const FString& Directory, const FDirectoryChanged& InDelegate, FDelegateHandle& OutHandle, uint32 Flags = 0) = 0;

	/**
	 * Unregisters a callback to fire when directories are changed
	 *
	 * @param	Directory to stop watching
	 * @param	Handle to the delegate to remove from our callback list
	 */
	virtual bool UnregisterDirectoryChangedCallback_Handle (const FString& Directory, FDelegateHandle InHandle) = 0;

	/**
	 * Allows for subclasses to be ticked (by editor or other programs that need to tick the singleton)
	 */
	virtual void Tick(float DeltaSeconds)
	{
	}

	/**
	 * Allows for subclasses to dump notify statistics (returns true if implemented)
	 */
	virtual bool DumpStats()
	{
		return false;
	}
};

